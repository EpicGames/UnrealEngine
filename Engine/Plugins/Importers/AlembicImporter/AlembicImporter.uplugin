{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Alembic Importer",
	"Description": "Support importing Alembic files",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/WorkingWithContent/Importing/AlembicImporter/",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AlembicImporter",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		},
		{
			"Name": "AlembicLibrary",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "GeometryCache",
			"Enabled": true
		}
	]
}
