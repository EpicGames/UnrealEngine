// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "EditorReimportHandler.h"
#include "AlembicImportFactory.generated.h"

#define UE_API ALEMBICIMPORTER_API

class UAbcImportSettings;
class FAbcImporter;
class UGeometryCache;
class UStaticMesh;
class USkeletalMesh;
class UAbcAssetImportData;
class SAlembicImportOptions;
class SAlembicTrackSelectionWindow;

class FAbcImporter;

UCLASS(MinimalAPI, hidecategories = Object)
class UAlembicImportFactory : public UFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	/** Object used to show import options for Alembic */
	UPROPERTY()
	TObjectPtr<UAbcImportSettings> ImportSettings;

	UPROPERTY()
	bool bShowOption;

	//~ Begin UObject Interface
	UE_API void PostInitProperties();
	//~ End UObject Interface

	//~ Begin UFactory Interface
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual bool DoesSupportClass(UClass * Class) override;
	UE_API virtual UClass* ResolveSupportedClass() override;
	UE_API virtual bool FactoryCanImport(const FString& Filename) override;
	UE_API virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	//~ End UFactory Interface

	//~ Begin FReimportHandler Interface
	UE_API virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	UE_API virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	UE_API virtual EReimportResult::Type Reimport(UObject* Obj) override;

	UE_API void ShowImportOptionsWindow(TSharedPtr<SAlembicImportOptions>& Options, FString FilePath, const FAbcImporter& Importer);

	UE_API virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface
		
	/**
	* Imports a StaticMesh (using AbcImporter) from the Alembic File
	*
	* @param Importer - AbcImporter instance
	* @param InParent - Parent for the StaticMesh
	* @param Flags - Creation Flags for the StaticMesh	
	* @return UObject*
	*/
	UE_API TArray<UObject*> ImportStaticMesh(FAbcImporter& Importer, UObject* InParent, EObjectFlags Flags);
	
	/**
	* ImportGeometryCache
	*	
	* @param Importer - AbcImporter instance
	* @param InParent - Parent for the GeometryCache asset
	* @param Flags - Creation Flags for the GeometryCache asset
	* @return UObject*
	*/
	UE_API UObject* ImportGeometryCache(FAbcImporter& Importer, UObject* InParent, EObjectFlags Flags);

	/**
	* ImportGeometryCache
	*	
	* @param Importer - AbcImporter instance
	* @param InParent - Parent for the GeometryCache asset
	* @param Flags - Creation Flags for the GeometryCache asset
	* @return UObject*
	*/
	UE_API TArray<UObject*> ImportSkeletalMesh(FAbcImporter& Importer, UObject* InParent, EObjectFlags Flags);

	/**
	* ReimportStaticMesh
	*
	* @param Mesh - Static mesh instance to re import
	* @return EReimportResult::Type
	*/
	UE_API EReimportResult::Type ReimportStaticMesh(UStaticMesh* Mesh);

	/**
	* ReimportGeometryCache
	*
	* @param Cache - Geometry cache instance to re import
	* @return EReimportResult::Type
	*/
	UE_API EReimportResult::Type ReimportGeometryCache(UGeometryCache* Cache);

	/**
	* ReimportGeometryCache
	*
	* @param Cache - Geometry cache instance to re import
	* @return EReimportResult::Type
	*/
	UE_API EReimportResult::Type ReimportSkeletalMesh(USkeletalMesh* SkeletalMesh);

	UE_API void PopulateOptionsWithImportData(UAbcAssetImportData* ImportData);
};

#undef UE_API
