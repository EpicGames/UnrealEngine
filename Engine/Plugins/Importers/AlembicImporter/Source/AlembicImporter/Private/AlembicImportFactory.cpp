// Copyright Epic Games, Inc. All Rights Reserved.

#include "AlembicImportFactory.h"
#include "AbcPolyMesh.h"
#include "AssetImportTask.h"
#include "Animation/AnimSequence.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/AssetUserData.h"
#include "Engine/SkeletalMesh.h"
#include "Editor.h"
#include "HAL/FileManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"

#include "AlembicImportOptions.h"
#include "AbcImporter.h"
#include "AbcImportLogger.h"
#include "AbcAssetImportData.h"

#include "GeometryCache.h"
#include "RenderingThread.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include "AI/Navigation/NavCollisionBase.h"
#include "ImportUtils/StaticMeshImportUtils.h"
#include "Subsystems/ImportSubsystem.h"
#include "UObject/UObjectIterator.h"
#include "AssetToolsModule.h"
#include "Misc/NamePermissionList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AlembicImportFactory)

#define LOCTEXT_NAMESPACE "AlembicImportFactory"

DEFINE_LOG_CATEGORY_STATIC(LogAlembic, Log, All);

UAlembicImportFactory::UAlembicImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	bEditAfterNew = true;
	SupportedClass = nullptr;

	bEditorImport = true;
	bText = false;

	bShowOption = true;

	Formats.Add(TEXT("abc;Alembic"));
}

void UAlembicImportFactory::PostInitProperties()
{
	Super::PostInitProperties();
	ImportSettings = UAbcImportSettings::Get();
}

FText UAlembicImportFactory::GetDisplayName() const
{
	return LOCTEXT("AlembicImportFactoryDescription", "Alembic");
}

bool UAlembicImportFactory::DoesSupportClass(UClass * Class)
{
	return (Class == UStaticMesh::StaticClass() || Class == UGeometryCache::StaticClass() || Class == USkeletalMesh::StaticClass() || Class == UAnimSequence::StaticClass());
}

UClass* UAlembicImportFactory::ResolveSupportedClass()
{
	return UStaticMesh::StaticClass();
}

bool UAlembicImportFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);
	return FPaths::GetExtension(Filename) == TEXT("abc");
}

UObject* UAlembicImportFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	const bool bIsUnattended = (IsAutomatedImport()
		|| FApp::IsUnattended()
		|| IsRunningCommandlet()
		|| GIsRunningUnattendedScript);
	bShowOption = !bIsUnattended;

	// Check if it's a re-import
	if (InParent != nullptr)
	{
		UObject* ExistingObject =
			StaticFindObject(UObject::StaticClass(), InParent, *InName.ToString());
		if (ExistingObject)
		{
			// Use this class as no other re-import handler exist for Alembics, yet
			FReimportHandler* ReimportHandler = this;
			TArray<FString> Filenames;
			Filenames.Add(UFactory::CurrentFilename);
			// Set the new source path before starting the re-import
			FReimportManager::Instance()->UpdateReimportPaths(ExistingObject, Filenames);
			// Do the re-import and exit
			const bool bIsAutomated = bIsUnattended;
			const bool bShowNotification = !bIsAutomated;
			const bool bAskForNewFileIfMissing = true;
			const FString PreferredReimportFile;
			const int32 SourceFileIndex = INDEX_NONE;
			const bool bForceNewFile = false;
			FReimportManager::Instance()->Reimport(
				ExistingObject,
				bAskForNewFileIfMissing,
				bShowNotification,
				PreferredReimportFile,
				ReimportHandler,
				SourceFileIndex,
				bForceNewFile,
				bIsAutomated);
			return ExistingObject;
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, TEXT("ABC"));

	// Use (and show) the settings from the script if provided
	UAbcImportSettings* ScriptedSettings = AssetImportTask ? Cast<UAbcImportSettings>(AssetImportTask->Options) : nullptr;
	if (ScriptedSettings)
	{
		ImportSettings = ScriptedSettings;
	}

	FAbcImporter Importer;
	EAbcImportError ErrorCode = Importer.OpenAbcFileForImport(Filename);
	ImportSettings->bReimport = false;
	AdditionalImportedObjects.Empty();

	// Set up message log page name to separate different assets
	FText ImportingText = FText::Format(LOCTEXT("AbcFactoryImporting", "Importing {0}.abc"), FText::FromName(InName));
	const FString& PageName = ImportingText.ToString();

	if (ErrorCode != AbcImportError_NoError)
	{
		FAbcImportLogger::OutputMessages(PageName);

		// Failed to read the file info, fail the import
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
		return nullptr;
	}

	if (ImportSettings == UAbcImportSettings::Get())
	{
		// Reset (possible) changed frame start value 
		ImportSettings->SamplingSettings.FrameStart = 0;
		ImportSettings->SamplingSettings.FrameEnd = Importer.GetEndFrameIndex();
	}
	else
	{
		// Don't override the frame end value from the script except if it was unset
		if (ImportSettings->SamplingSettings.FrameEnd == 0)
		{
			ImportSettings->SamplingSettings.FrameEnd = Importer.GetEndFrameIndex();
		}
	}

	bOutOperationCanceled = false;

	if (bShowOption)
	{
		TSharedPtr<SAlembicImportOptions> Options;
		ShowImportOptionsWindow(Options, UFactory::CurrentFilename, Importer);
		// Set whether or not the user canceled
		bOutOperationCanceled = !Options->ShouldImport();
	}


	TArray<UObject*> ResultAssets;
	if (!bOutOperationCanceled)
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, TEXT("ABC"));

		int32 NumThreads = 1;
		if (FPlatformProcess::SupportsMultithreading())
		{
			NumThreads = FPlatformMisc::NumberOfCores();
		}

		// Import file		
		ErrorCode = Importer.ImportTrackData(NumThreads, ImportSettings);

		if (ErrorCode != AbcImportError_NoError)
		{
			// Failed to read the file info, fail the import
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			FAbcImportLogger::OutputMessages(PageName);
			return nullptr;
		}
		else
		{
			static bool bGeometryCacheAllowed = []() -> bool
			{
				IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
				TSharedPtr<FPathPermissionList> AssetClassPermissionList = AssetTools.GetAssetClassPathPermissionList(EAssetClassAction::ImportAsset);
				if (AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
				{
					return AssetClassPermissionList->PassesFilter(TEXT("/Script/GeometryCache.GeometryCache"));
				}

				return true;
			}();

			if (ImportSettings->ImportType == EAlembicImportType::StaticMesh)
			{
				const TArray<UObject*> ResultStaticMeshes = ImportStaticMesh(Importer, InParent, Flags);
				ResultAssets.Append(ResultStaticMeshes);
			}
			else if (bGeometryCacheAllowed && ImportSettings->ImportType == EAlembicImportType::GeometryCache)
			{
				UObject* GeometryCache = ImportGeometryCache(Importer, InParent, Flags);
				if (GeometryCache)
				{
					ResultAssets.Add(GeometryCache);
				}
			}
			else if (ImportSettings->ImportType == EAlembicImportType::Skeletal)
			{
				TArray<UObject*> SkeletalMesh = ImportSkeletalMesh(Importer, InParent, Flags);
				ResultAssets.Append(SkeletalMesh);
			}
		}

		AdditionalImportedObjects.Reserve(ResultAssets.Num());
		for (UObject* Object : ResultAssets)
		{
			if (Object)
			{
				FAssetRegistryModule::AssetCreated(Object);
				GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Object);
				// Avoid redoing a PostEditChange since the importer will have most likely done it already
				if (!Object->GetOutermost()->IsDirty())
				{
					Object->MarkPackageDirty();
					Object->PostEditChange();
				}
				AdditionalImportedObjects.Add(Object);
			}
		}

		FAbcImportLogger::OutputMessages(PageName);
	}

	return (ResultAssets.Num() > 0) ? ResultAssets[0] : nullptr;
}

TArray<UObject*> UAlembicImportFactory::ImportStaticMesh(FAbcImporter& Importer, UObject* InParent, EObjectFlags Flags)
{
	// Flush commands before importing
	FlushRenderingCommands();

	TArray<UObject*> Objects;

	const uint32 NumMeshes = Importer.GetNumMeshTracks();
	// Check if the alembic file contained any meshes
	if (NumMeshes > 0)
	{
		const TArray<UStaticMesh*>& StaticMeshes = Importer.ImportAsStaticMesh(InParent, Flags);

		for (UStaticMesh* StaticMesh : StaticMeshes)
		{
			if (StaticMesh)
			{
				// Setup asset import data
				if (!StaticMesh->GetAssetImportData() || !StaticMesh->GetAssetImportData()->IsA<UAbcAssetImportData>())
				{
					StaticMesh->SetAssetImportData(NewObject<UAbcAssetImportData>(StaticMesh));
				}
				StaticMesh->GetAssetImportData()->Update(UFactory::CurrentFilename);

				UAbcAssetImportData* AssetImportData = Cast<UAbcAssetImportData>(StaticMesh->GetAssetImportData());
				if (AssetImportData)
				{
					Importer.UpdateAssetImportData(AssetImportData);
				}

				Objects.Add(StaticMesh);
			}
		}
	}

	return Objects;
}

UObject* UAlembicImportFactory::ImportGeometryCache(FAbcImporter& Importer, UObject* InParent, EObjectFlags Flags)
{
	// Flush commands before importing
	FlushRenderingCommands();

	const uint32 NumMeshes = Importer.GetNumMeshTracks();
	// Check if the alembic file contained any meshes
	if (NumMeshes > 0)
	{
		UGeometryCache* GeometryCache = Importer.ImportAsGeometryCache(InParent, Flags);

		if (!GeometryCache)
		{
			return nullptr;
		}

		// Setup asset import data
		if (!GeometryCache->AssetImportData || !GeometryCache->AssetImportData->IsA<UAbcAssetImportData>())
		{
			GeometryCache->AssetImportData = NewObject<UAbcAssetImportData>(GeometryCache);
		}
		GeometryCache->AssetImportData->Update(UFactory::CurrentFilename);
		UAbcAssetImportData* AssetImportData = Cast<UAbcAssetImportData>(GeometryCache->AssetImportData);
		if (AssetImportData)
		{
			Importer.UpdateAssetImportData(AssetImportData);
		}

		return GeometryCache;
	}
	else
	{
		// Not able to import a static mesh
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
		return nullptr;
	}
}

TArray<UObject*> UAlembicImportFactory::ImportSkeletalMesh(FAbcImporter& Importer, UObject* InParent, EObjectFlags Flags)
{
	// Flush commands before importing
	FlushRenderingCommands();

	const uint32 NumMeshes = Importer.GetNumMeshTracks();
	// Check if the alembic file contained any meshes
	if (NumMeshes > 0)
	{
		TArray<UObject*> GeneratedObjects = Importer.ImportAsSkeletalMesh(InParent, Flags);

		if (!GeneratedObjects.Num())
		{
			return {};
		}

		USkeletalMesh* SkeletalMesh = [&GeneratedObjects]()
		{
			UObject** FoundObject = GeneratedObjects.FindByPredicate([](UObject* Object) { return Object->IsA<USkeletalMesh>(); });
			return FoundObject ? CastChecked<USkeletalMesh>(*FoundObject) : nullptr;
		}();			
			
		if (SkeletalMesh)
		{
			// Setup asset import data
			if (!SkeletalMesh->GetAssetImportData() || !SkeletalMesh->GetAssetImportData()->IsA<UAbcAssetImportData>())
			{
				SkeletalMesh->SetAssetImportData(NewObject<UAbcAssetImportData>(SkeletalMesh));
			}
			SkeletalMesh->GetAssetImportData()->Update(UFactory::CurrentFilename);
			UAbcAssetImportData* AssetImportData = Cast<UAbcAssetImportData>(SkeletalMesh->GetAssetImportData());
			if (AssetImportData)
			{
				Importer.UpdateAssetImportData(AssetImportData);
			}
		}

		UAnimSequence* AnimSequence = [&GeneratedObjects]()
		{
			UObject** FoundObject = GeneratedObjects.FindByPredicate([](UObject* Object) { return Object->IsA<UAnimSequence>(); });
			return FoundObject ? CastChecked<UAnimSequence>(*FoundObject) : nullptr;
		}();

		if (AnimSequence)
		{
			// Setup asset import data
			if (!AnimSequence->AssetImportData || !AnimSequence->AssetImportData->IsA<UAbcAssetImportData>())
			{
				AnimSequence->AssetImportData = NewObject<UAbcAssetImportData>(AnimSequence);
			}
			AnimSequence->AssetImportData->Update(UFactory::CurrentFilename);
			UAbcAssetImportData* AssetImportData = Cast<UAbcAssetImportData>(AnimSequence->AssetImportData);
			if (AssetImportData)
			{
				Importer.UpdateAssetImportData(AssetImportData);
			}
		}
		
		return GeneratedObjects;
	}
	else
	{
		// Not able to import as skeletal mesh
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
		return {};
	}
}

bool UAlembicImportFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UAssetImportData* ImportData = nullptr;
	if (Obj->GetClass() == UStaticMesh::StaticClass())
	{
		UStaticMesh* Mesh = Cast<UStaticMesh>(Obj);
		ImportData = Mesh->GetAssetImportData();
	}
	else if (Obj->GetClass() == UGeometryCache::StaticClass())
	{
		UGeometryCache* Cache = Cast<UGeometryCache>(Obj);
		ImportData = Cache->AssetImportData;
	}
	else if (Obj->GetClass() == USkeletalMesh::StaticClass())
	{
		USkeletalMesh* Cache = Cast<USkeletalMesh>(Obj);
		ImportData = Cache->GetAssetImportData();
	}
	else if (Obj->GetClass() == UAnimSequence::StaticClass())
	{
		UAnimSequence* Cache = Cast<UAnimSequence>(Obj);
		ImportData = Cache->AssetImportData;
	}
	
	if (ImportData)
	{
		if (FPaths::GetExtension(ImportData->GetFirstFilename()) == TEXT("abc") || ( Obj->GetClass() == UAnimSequence::StaticClass() && ImportData->GetFirstFilename().IsEmpty()))
		{
			ImportData->ExtractFilenames(OutFilenames);
			return true;
		}
	}
	return false;
}

void UAlembicImportFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UStaticMesh* Mesh = Cast<UStaticMesh>(Obj);
	if (Mesh && Mesh->GetAssetImportData() && ensure(NewReimportPaths.Num() == 1))
	{
		Mesh->GetAssetImportData()->UpdateFilenameOnly(NewReimportPaths[0]);
	}

	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Obj);
	if (SkeletalMesh && SkeletalMesh->GetAssetImportData() && ensure(NewReimportPaths.Num() == 1))
	{
		SkeletalMesh->GetAssetImportData()->UpdateFilenameOnly(NewReimportPaths[0]);
	}

	UAnimSequence* Sequence = Cast<UAnimSequence>(Obj);
	if (Sequence && Sequence->AssetImportData && ensure(NewReimportPaths.Num() == 1))
	{
		Sequence->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}

	UGeometryCache* GeometryCache = Cast<UGeometryCache>(Obj);
	if (GeometryCache && GeometryCache->AssetImportData && ensure(NewReimportPaths.Num() == 1))
	{
		GeometryCache->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UAlembicImportFactory::Reimport(UObject* Obj)
{
	ImportSettings->bReimport = true;

	const bool bIsUnattended = (IsAutomatedImport()
		|| FApp::IsUnattended()
		|| IsRunningCommandlet()
		|| GIsRunningUnattendedScript);
	bShowOption = !bIsUnattended;

	FText ReimportingText = FText::Format(LOCTEXT("AbcFactoryReimporting", "Reimporting {0}.abc"), FText::FromString(Obj->GetName()));
	const FString& PageName = ReimportingText.ToString();
	if (Obj->GetClass() == UStaticMesh::StaticClass())
	{
		UStaticMesh* Mesh = Cast<UStaticMesh>(Obj);
		if (!Mesh)
		{
			return EReimportResult::Failed;
		}

		CurrentFilename = Mesh->GetAssetImportData()->GetFirstFilename();

		// Close possible open editors using this asset	
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(Mesh);
		
		EReimportResult::Type Result = ReimportStaticMesh(Mesh);
		FAbcImportLogger::OutputMessages(PageName);

		return Result;
	}
	else if (Obj->GetClass() == UGeometryCache::StaticClass())
	{
		UGeometryCache* GeometryCache = Cast<UGeometryCache>(Obj);
		if (!GeometryCache)
		{
			return EReimportResult::Failed;
		}

		CurrentFilename = GeometryCache->AssetImportData->GetFirstFilename();

		EReimportResult::Type Result = ReimportGeometryCache(GeometryCache);

		if (GeometryCache->GetOuter())
		{
			GeometryCache->GetOuter()->MarkPackageDirty();
		}
		else
		{
			GeometryCache->MarkPackageDirty();
		}

		// Close possible open editors using this asset	
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(GeometryCache);

		FAbcImportLogger::OutputMessages(PageName);
		return Result;
	}
	else if (Obj->GetClass() == USkeletalMesh::StaticClass())
	{
		USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Obj);
		if (!SkeletalMesh)
		{
			return EReimportResult::Failed;
		}

		CurrentFilename = SkeletalMesh->GetAssetImportData()->GetFirstFilename();

		EReimportResult::Type Result = ReimportSkeletalMesh(SkeletalMesh);

		if (SkeletalMesh->GetOuter())
		{
			SkeletalMesh->GetOuter()->MarkPackageDirty();
		}
		else
		{
			SkeletalMesh->MarkPackageDirty();
		}

		// Close possible open editors using this asset	
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(SkeletalMesh);

		FAbcImportLogger::OutputMessages(PageName);
		return Result;
	}
	else if (Obj->GetClass() == UAnimSequence::StaticClass())
	{
		UAnimSequence* AnimSequence = Cast<UAnimSequence>(Obj);
		if (!AnimSequence)
		{
			return EReimportResult::Failed;
		}

		CurrentFilename = AnimSequence->AssetImportData->GetFirstFilename();
		USkeletalMesh* SkeletalMesh = nullptr;
		for (TObjectIterator<USkeletalMesh> It; It; ++It)
		{
			// This works because the skeleton is unique for every imported alembic cache
			if (It->GetSkeleton() == AnimSequence->GetSkeleton())
			{
				SkeletalMesh = *It;
				break;
			}
		}

		if (!SkeletalMesh)
		{
			return EReimportResult::Failed;
		}

		// Close possible open editors using this asset	
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(AnimSequence);

		EReimportResult::Type Result = ReimportSkeletalMesh(SkeletalMesh);

		if (SkeletalMesh->GetOuter())
		{
			SkeletalMesh->GetOuter()->MarkPackageDirty();
		}
		else
		{
			SkeletalMesh->MarkPackageDirty();
		}

		FAbcImportLogger::OutputMessages(PageName);
		return Result;
	}

	return EReimportResult::Failed;
}

void UAlembicImportFactory::ShowImportOptionsWindow(TSharedPtr<SAlembicImportOptions>& Options, FString FilePath, const FAbcImporter& Importer)
{
	// Window size computed from SAlembicImportOptions
	const float WindowHeight = 500.f + FMath::Clamp(Importer.GetPolyMeshes().Num() * 16.f, 0.f, 250.f);

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("WindowTitle", "Alembic Cache Import Options"))
		.ClientSize(FVector2D(522.f, WindowHeight));

	Window->SetContent
		(
			SAssignNew(Options, SAlembicImportOptions).WidgetWindow(Window)
			.ImportSettings(ImportSettings)
			.WidgetWindow(Window)
			.PolyMeshes(Importer.GetPolyMeshes())
			.FullPath(FText::FromString(FilePath))
		);

	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
}

EReimportResult::Type UAlembicImportFactory::ReimportGeometryCache(UGeometryCache* Cache)
{
	// Ensure that the file provided by the path exists
	if (IFileManager::Get().FileSize(*CurrentFilename) == INDEX_NONE)
	{
		return EReimportResult::Failed;
	}

	FAbcImporter Importer;
	EAbcImportError ErrorCode = Importer.OpenAbcFileForImport(CurrentFilename);

	if (ErrorCode != AbcImportError_NoError)
	{
		// Failed to read the file info, fail the re importing process 
		return EReimportResult::Failed;
	}
	
	ImportSettings->ImportType = EAlembicImportType::GeometryCache;
	ImportSettings->SamplingSettings.FrameStart = 0;
	ImportSettings->SamplingSettings.FrameEnd = Importer.GetEndFrameIndex();

	if (Cache->AssetImportData && Cache->AssetImportData->IsA<UAbcAssetImportData>())
	{
		UAbcAssetImportData* ImportData = Cast<UAbcAssetImportData>(Cache->AssetImportData);
		PopulateOptionsWithImportData(ImportData);
		Importer.RetrieveAssetImportData(ImportData);
	}

	if (bShowOption)
	{
		TSharedPtr<SAlembicImportOptions> Options;
		ShowImportOptionsWindow(Options, CurrentFilename, Importer);

		if (!Options->ShouldImport())
		{
			return EReimportResult::Cancelled;
		}
	}
	else
	{
		UAbcImportSettings* ScriptedSettings = AssetImportTask
			? Cast<UAbcImportSettings>(AssetImportTask->Options)
			: nullptr;
		if (ScriptedSettings)
		{
			ImportSettings = ScriptedSettings;
		}
	}

	int32 NumThreads = 1;
	if (FPlatformProcess::SupportsMultithreading())
	{
		NumThreads = FPlatformMisc::NumberOfCores();
	}

	// Import file	
	ErrorCode = Importer.ImportTrackData(NumThreads, ImportSettings);

	if (ErrorCode != AbcImportError_NoError)
	{
		// Failed to read the file info, fail the re importing process 
		return EReimportResult::Failed;
	}

	UGeometryCache* GeometryCache = Importer.ReimportAsGeometryCache(Cache);

	if (!GeometryCache)
	{
		return EReimportResult::Failed;
	}
	else
	{
		// Update file path/timestamp (Path could change if user has to browse for it manually)
		if (!GeometryCache->AssetImportData || !GeometryCache->AssetImportData->IsA<UAbcAssetImportData>())
		{
			GeometryCache->AssetImportData = NewObject<UAbcAssetImportData>(GeometryCache);
		}

		GeometryCache->AssetImportData->Update(CurrentFilename);
		UAbcAssetImportData* AssetImportData = Cast<UAbcAssetImportData>(GeometryCache->AssetImportData);
		if (AssetImportData)
		{
			Importer.UpdateAssetImportData(AssetImportData);
		}
	}

	return EReimportResult::Succeeded;
}

EReimportResult::Type UAlembicImportFactory::ReimportSkeletalMesh(USkeletalMesh* SkeletalMesh)
{
	// Ensure that the file provided by the path exists
	if (IFileManager::Get().FileSize(*CurrentFilename) == INDEX_NONE)
	{
		return EReimportResult::Failed;
	}

	FAbcImporter Importer;
	EAbcImportError ErrorCode = Importer.OpenAbcFileForImport(CurrentFilename);

	if (ErrorCode != AbcImportError_NoError)
	{
		// Failed to read the file info, fail the re importing process 
		return EReimportResult::Failed;
	}

	if (SkeletalMesh->GetAssetImportData() && SkeletalMesh->GetAssetImportData()->IsA<UAbcAssetImportData>())
	{
		UAbcAssetImportData* ImportData = Cast<UAbcAssetImportData>(SkeletalMesh->GetAssetImportData());
		PopulateOptionsWithImportData(ImportData);
		Importer.RetrieveAssetImportData(ImportData);
	}

	ImportSettings->ImportType = EAlembicImportType::Skeletal;
	ImportSettings->SamplingSettings.FrameStart = 0;
	ImportSettings->SamplingSettings.FrameEnd = Importer.GetEndFrameIndex();

	if (bShowOption)
	{
		TSharedPtr<SAlembicImportOptions> Options;
		ShowImportOptionsWindow(Options, CurrentFilename, Importer);

		if (!Options->ShouldImport())
		{
			return EReimportResult::Cancelled;
		}
	}
	else
	{
		UAbcImportSettings* ScriptedSettings = AssetImportTask
			? Cast<UAbcImportSettings>(AssetImportTask->Options)
			: nullptr;
		if (ScriptedSettings)
		{
			ImportSettings = ScriptedSettings;
		}
	}

	int32 NumThreads = 1;
	if (FPlatformProcess::SupportsMultithreading())
	{
		NumThreads = FPlatformMisc::NumberOfCores();
	}

	// Import file	
	ErrorCode = Importer.ImportTrackData(NumThreads, ImportSettings);

	if (ErrorCode != AbcImportError_NoError)
	{
		// Failed to read the file info, fail the re importing process 
		return EReimportResult::Failed;
	}

	TArray<UObject*> ReimportedObjects = Importer.ReimportAsSkeletalMesh(SkeletalMesh);
	USkeletalMesh* NewSkeletalMesh = [&ReimportedObjects]()
	{
		UObject** FoundObject = ReimportedObjects.FindByPredicate([](UObject* Object) { return Object->IsA<USkeletalMesh>(); });
		return FoundObject ? CastChecked<USkeletalMesh>(*FoundObject) : nullptr;
	}();

	if (!NewSkeletalMesh)
	{
		return EReimportResult::Failed;
	}
	else
	{
		// Update file path/timestamp (Path could change if user has to browse for it manually)
		if (!NewSkeletalMesh->GetAssetImportData() || !NewSkeletalMesh->GetAssetImportData()->IsA<UAbcAssetImportData>())
		{
			NewSkeletalMesh->SetAssetImportData(NewObject<UAbcAssetImportData>(NewSkeletalMesh));
		}

		NewSkeletalMesh->GetAssetImportData()->Update(CurrentFilename);
		UAbcAssetImportData* AssetImportData = Cast<UAbcAssetImportData>(NewSkeletalMesh->GetAssetImportData());
		if (AssetImportData)
		{
			Importer.UpdateAssetImportData(AssetImportData);
		}
	}

	UAnimSequence* NewAnimSequence = [&ReimportedObjects]()
	{
		UObject** FoundObject = ReimportedObjects.FindByPredicate([](UObject* Object) { return Object->IsA<UAnimSequence>(); });
		return FoundObject ? CastChecked<UAnimSequence>(*FoundObject) : nullptr;
	}();

	if (!NewAnimSequence)
	{
		return EReimportResult::Failed;
	}
	else
	{
		// Update file path/timestamp (Path could change if user has to browse for it manually)
		if (!NewAnimSequence->AssetImportData || !NewAnimSequence->AssetImportData->IsA<UAbcAssetImportData>())
		{
			NewAnimSequence->AssetImportData = NewObject<UAbcAssetImportData>(NewAnimSequence);
		}

		NewAnimSequence->AssetImportData->Update(CurrentFilename);
		UAbcAssetImportData* AssetImportData = Cast<UAbcAssetImportData>(NewAnimSequence->AssetImportData);
		if (AssetImportData)
		{
			Importer.UpdateAssetImportData(AssetImportData);
		}
	}

	return EReimportResult::Succeeded;
}

void UAlembicImportFactory::PopulateOptionsWithImportData(UAbcAssetImportData* ImportData)
{
	ImportSettings->SamplingSettings = ImportData->SamplingSettings;
	ImportSettings->NormalGenerationSettings = ImportData->NormalGenerationSettings;
	ImportSettings->CompressionSettings = ImportData->CompressionSettings;
	ImportSettings->StaticMeshSettings = ImportData->StaticMeshSettings;
	ImportSettings->GeometryCacheSettings = ImportData->GeometryCacheSettings;
	ImportSettings->ConversionSettings = ImportData->ConversionSettings;
}

EReimportResult::Type UAlembicImportFactory::ReimportStaticMesh(UStaticMesh* Mesh)
{
	// Ensure that the file provided by the path exists
	if (IFileManager::Get().FileSize(*CurrentFilename) == INDEX_NONE)
	{
		return EReimportResult::Failed;
	}

	FAbcImporter Importer;
	EAbcImportError ErrorCode = Importer.OpenAbcFileForImport(CurrentFilename);

	if (ErrorCode != AbcImportError_NoError)
	{
		// Failed to read the file info, fail the re importing process 
		return EReimportResult::Failed;
	}

	if (Mesh->GetAssetImportData() && Mesh->GetAssetImportData()->IsA<UAbcAssetImportData>())
	{
		UAbcAssetImportData* ImportData = Cast<UAbcAssetImportData>(Mesh->GetAssetImportData());
		PopulateOptionsWithImportData(ImportData);
		Importer.RetrieveAssetImportData(ImportData);
	}

	ImportSettings->ImportType = EAlembicImportType::StaticMesh;
	ImportSettings->SamplingSettings.FrameStart = 0;
	ImportSettings->SamplingSettings.FrameEnd = Importer.GetEndFrameIndex();

	if (bShowOption)
	{
		TSharedPtr<SAlembicImportOptions> Options;
		ShowImportOptionsWindow(Options, CurrentFilename, Importer);

		if (!Options->ShouldImport())
		{
			return EReimportResult::Cancelled;
		}
	}
	else
	{
		UAbcImportSettings* ScriptedSettings = AssetImportTask
			? Cast<UAbcImportSettings>(AssetImportTask->Options)
			: nullptr;
		if (ScriptedSettings)
		{
			ImportSettings = ScriptedSettings;
		}
	}

	int32 NumThreads = 1;
	if (FPlatformProcess::SupportsMultithreading())
	{
		NumThreads = FPlatformMisc::NumberOfCores();
	}

	ErrorCode = Importer.ImportTrackData(NumThreads, ImportSettings);

	if (ErrorCode != AbcImportError_NoError)
	{
		// Failed to read the file info, fail the re importing process 
		return EReimportResult::Failed;
	}
	else
	{
		// Preserve settings on the existing mesh (same as on a FBX re-import)
		const bool bOverrideExistingMaterials = false;
		TSharedPtr<FExistingStaticMeshData> ExistingMeshData =
			StaticMeshImportUtils::SaveExistingStaticMeshData(Mesh, bOverrideExistingMaterials, INDEX_NONE);

		// Preserve the user data
		const TArray<UAssetUserData*>* UserData = Mesh->GetAssetUserDataArray();
		TMap<UAssetUserData*, bool> UserDataCopy;
		if (UserData)
		{
			for (int32 Idx = 0; Idx < UserData->Num(); Idx++)
			{
				if ((*UserData)[Idx] != nullptr)
				{
					// This is slightly different from the equivalent in the FBX
					// reimporter as it is using the deprecated version of the same
					// method (StaticDuplicateObject).
					FObjectDuplicationParameters DupParams(
						(*UserData)[Idx], GetTransientPackage());
					UAssetUserData* DupObject =
						Cast<UAssetUserData>(StaticDuplicateObjectEx(DupParams));
					bool bAddDupToRoot = !DupObject->IsRooted();
					if (bAddDupToRoot)
					{
						DupObject->AddToRoot();
					}
					UserDataCopy.Add(DupObject, bAddDupToRoot);
				}
			}
		}

		// Preserve settings in navcollision subobject
		UNavCollisionBase* NavCollision = nullptr;
		if (Mesh->GetNavCollision())
		{
			FObjectDuplicationParameters DupParams(
				Mesh->GetNavCollision(), GetTransientPackage());
			NavCollision = Cast<UNavCollisionBase>(StaticDuplicateObjectEx(DupParams));
		}

		bool bAddedNavCollisionDupToRoot = false;
		if (NavCollision && !NavCollision->IsRooted())
		{
			bAddedNavCollisionDupToRoot = true;
			NavCollision->AddToRoot();
		}

		const TArray<UStaticMesh*>& StaticMeshes = Importer.ReimportAsStaticMesh(Mesh);
		for (UStaticMesh* StaticMesh : StaticMeshes)
		{
			if (StaticMesh)
			{
				// Setup asset import data
				if (!StaticMesh->GetAssetImportData() || !StaticMesh->GetAssetImportData()->IsA<UAbcAssetImportData>())
				{
					StaticMesh->SetAssetImportData(NewObject<UAbcAssetImportData>(StaticMesh));
				}
				StaticMesh->GetAssetImportData()->Update(UFactory::CurrentFilename);
				UAbcAssetImportData* AssetImportData = Cast<UAbcAssetImportData>(StaticMesh->GetAssetImportData());
				if (AssetImportData)
				{
					Importer.UpdateAssetImportData(AssetImportData);
				}

				// Restore preserved user data
				for (TPair<UAssetUserData*, bool> Kvp : UserDataCopy)
				{
					UAssetUserData* UserDataObject = Kvp.Key;
					if (Kvp.Value)
					{
						// If the duplicated temporary UObject was added to root,
						// we must remove it from the root
						UserDataObject->RemoveFromRoot();
					}
					UserDataObject->Rename(nullptr, Mesh,
						REN_DontCreateRedirectors | REN_DoNotDirty);
					StaticMesh->AddAssetUserData(UserDataObject);
				}

				// Restore navcollision subobject
				if (NavCollision)
				{
					if (bAddedNavCollisionDupToRoot)
					{
						// If the duplicated temporary UObject was added to root,
						// we must remove it from the root
						NavCollision->RemoveFromRoot();
					}
					StaticMesh->SetNavCollision(NavCollision);
					NavCollision->Rename(NULL, Mesh,
						REN_DontCreateRedirectors | REN_DoNotDirty);
				}

				// Restore additional mesh data settings
				StaticMeshImportUtils::RestoreExistingMeshData(
					ExistingMeshData,
					StaticMesh,
					INDEX_NONE,
					false,
					false);
			}
		}

		if (!StaticMeshes.Num())
		{
			return EReimportResult::Failed;
		}
	}

	return EReimportResult::Succeeded;
}

int32 UAlembicImportFactory::GetPriority() const
{
	return ImportPriority;
}

#undef LOCTEXT_NAMESPACE

