// Copyright Epic Games, Inc. All Rights Reserved.

#include "AlembicImportOptions.h"

#include "DetailsViewArgs.h"
#include "PropertyEditorModule.h"
#include "Framework/Views/TableViewMetadata.h"
#include "IDetailsView.h"
#include "AbcImportSettings.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"

#include "STrackSelectionTableRow.h"

#define LOCTEXT_NAMESPACE "AlembicImportOptions"

void SAlembicImportOptions::Construct(const FArguments& InArgs)
{
	ImportSettings = InArgs._ImportSettings;
	WidgetWindow = InArgs._WidgetWindow;

	check(ImportSettings);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ColumnWidth = 0.5f;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(ImportSettings);
	
	for (FAbcPolyMesh* PolyMesh : InArgs._PolyMeshes)
	{
		PolyMeshData.Add(FPolyMeshDataPtr(new FPolyMeshData(PolyMesh)));
	}

	static const float MaxDesiredHeight = 250.f;
	float MinDesiredHeight = FMath::Clamp(InArgs._PolyMeshes.Num() * 16.f, 0.f, MaxDesiredHeight);

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
					.Text(LOCTEXT("Import_CurrentFileTitle", "Current File: "))
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
					.Text(InArgs._FullPath)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SBox)				
				.MinDesiredWidth(512.0f)
				.MinDesiredHeight(MinDesiredHeight)
				.MaxDesiredHeight(MaxDesiredHeight)
				[
					SNew(SListView<FPolyMeshDataPtr>)
					.ScrollbarVisibility(EVisibility::Visible)
					.ListItemsSource(&PolyMeshData)
					.OnMouseButtonDoubleClick(this, &SAlembicImportOptions::OnItemDoubleClicked)
					.OnGenerateRow(this, &SAlembicImportOptions::OnGenerateWidgetForList)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column("ShouldImport")
						.FillWidth(0.1f)
						.DefaultLabel(FText::FromString(TEXT("Include")))
						[
							SNew(SCheckBox)
							.HAlign(HAlign_Center)
							.OnCheckStateChanged(this, &SAlembicImportOptions::OnToggleAllItems)
						]

						+ SHeaderRow::Column("TrackName")
						.DefaultLabel(LOCTEXT("TrackNameHeader", "Track Name"))
						.FillWidth(0.45f)
							
						+ SHeaderRow::Column("TrackFrameStart")
						.DefaultLabel(LOCTEXT("TrackFrameStartHeader", "Start Frame"))
						.FillWidth(0.15f)

						+ SHeaderRow::Column("TrackFrameEnd")
						.DefaultLabel(LOCTEXT("TrackFrameEndHeader", "End Frame"))
						.FillWidth(0.15f)

						+ SHeaderRow::Column("TrackFrameNum")
						.DefaultLabel(LOCTEXT("TrackFrameNumHeader", "Num Frames"))
						.FillWidth(0.15f)
					)
				]
			]
		]


		+ SVerticalBox::Slot()
		.Padding(2)
		.MaxHeight(500.0f)
		[
			DetailsView->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2)
			+ SUniformGridPanel::Slot(0, 0)
			[
				SAssignNew(ImportButton, SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("AlembicOptionWindow_Import", "Import"))
				.IsEnabled(this, &SAlembicImportOptions::CanImport)
				.OnClicked(this, &SAlembicImportOptions::OnImport)
			]
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("AlembicOptionWindow_Cancel", "Cancel"))
				.ToolTipText(LOCTEXT("AlembicOptionWindow_Cancel_ToolTip", "Cancels importing this Alembic file"))
				.OnClicked(this, &SAlembicImportOptions::OnCancel)
			]
		]
	];
}

TSharedRef<ITableRow> SAlembicImportOptions::OnGenerateWidgetForList(FPolyMeshDataPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STrackSelectionTableRow, OwnerTable)
		.PolyMesh(InItem);
}

bool SAlembicImportOptions::CanImport()  const
{
	return true;
}

void SAlembicImportOptions::OnToggleAllItems(ECheckBoxState CheckType)
{
	/** Set all items to top level checkbox state */
	for (FPolyMeshDataPtr& Item : PolyMeshData)
	{
		Item->PolyMesh->bShouldImport = CheckType == ECheckBoxState::Checked;
	}
}

void SAlembicImportOptions::OnItemDoubleClicked(FPolyMeshDataPtr ClickedItem)
{
	/** Toggle state on / off for the selected list entry */
	for (FPolyMeshDataPtr& Item : PolyMeshData)
	{
		if (Item == ClickedItem)
		{
			Item->PolyMesh->bShouldImport = !Item->PolyMesh->bShouldImport;
		}
	}
}

#undef LOCTEXT_NAMESPACE

