// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"

class FAbcPolyMesh;
class ITableRow;
class SButton;
class STableViewBase;
enum class ECheckBoxState : uint8;

// Forward declares
class UAbcImportSettings;
class IDetailsView;


struct FPolyMeshData
{
	FPolyMeshData(class FAbcPolyMesh* InPolyMesh) : PolyMesh(InPolyMesh) {}
	class FAbcPolyMesh* PolyMesh;
};

typedef TSharedPtr<FPolyMeshData> FPolyMeshDataPtr;

class SAlembicImportOptions : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAlembicImportOptions)
		: _ImportSettings(nullptr)
		, _WidgetWindow()
		{}

		SLATE_ARGUMENT(UAbcImportSettings*, ImportSettings)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(FText, FullPath)				
		SLATE_ARGUMENT(TArray<class FAbcPolyMesh*>, PolyMeshes)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnImport()
	{
		bShouldImport = true;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		bShouldImport = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldImport() const
	{
		return bShouldImport;
	}


	SAlembicImportOptions()
	: ImportSettings(nullptr)
	, bShouldImport(false)
	{}

private:
	TSharedRef<ITableRow> OnGenerateWidgetForList(FPolyMeshDataPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);
	bool CanImport() const;

	void OnToggleAllItems(ECheckBoxState CheckType);
	void OnItemDoubleClicked(FPolyMeshDataPtr ClickedItem);
private:
	UAbcImportSettings*	ImportSettings;
	TWeakPtr< SWindow > WidgetWindow;
	TSharedPtr< SButton > ImportButton;
	bool			bShouldImport;
	TArray<FPolyMeshDataPtr> PolyMeshData;
	TSharedPtr<IDetailsView> DetailsView;
};
