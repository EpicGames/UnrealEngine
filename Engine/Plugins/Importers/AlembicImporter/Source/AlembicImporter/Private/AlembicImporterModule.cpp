// Copyright Epic Games, Inc. All Rights Reserved.

#include "AlembicImporterModule.h"
#include "AlembicLibraryModule.h"

class FAlembicImporterModule : public IAlembicImporterModuleInterface
{
	virtual void StartupModule() override
	{
		FModuleManager::LoadModuleChecked< IAlembicLibraryModule >("AlembicLibrary");
	}
};

IMPLEMENT_MODULE(FAlembicImporterModule, AlembicImporter);