// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "SlateOptMacros.h"
#include "AbcPolyMesh.h"
#include "AlembicImportOptions.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"

struct FAbcTrackInformation;

typedef TSharedPtr<FPolyMeshData> FPolyMeshDataPtr;

/**
* Implements a row widget for the dispatch state list.
*/
class STrackSelectionTableRow
	: public SMultiColumnTableRow<FPolyMeshDataPtr>
{
public:

	SLATE_BEGIN_ARGS(STrackSelectionTableRow) { }
	SLATE_ARGUMENT(FPolyMeshDataPtr, PolyMesh)
	SLATE_END_ARGS()

public:

	/**
	* Constructs the widget.
	*
	* @param InArgs The construction arguments.
	* @param InOwnerTableView The table view that owns this row.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		check(InArgs._PolyMesh.IsValid());

		PolyMeshData = InArgs._PolyMesh;

		SMultiColumnTableRow<FPolyMeshDataPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	// SMultiColumnTableRow interface

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == "ShouldImport")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(this, &STrackSelectionTableRow::ShouldImportEnabled)
					.OnCheckStateChanged(this, &STrackSelectionTableRow::OnChangeShouldImport)					
				];
		}
		else if (ColumnName == "TrackName")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(PolyMeshData->PolyMesh->GetName()))
				];
		}
		else if (ColumnName == "TrackFrameStart")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(FString::FromInt(PolyMeshData->PolyMesh->GetFrameIndexForFirstData())))
				];
		}
		else if (ColumnName == "TrackFrameEnd")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(FString::FromInt(PolyMeshData->PolyMesh->GetFrameIndexForFirstData() + (PolyMeshData->PolyMesh->GetNumberOfSamples()-1))))
				];
		}
		else if (ColumnName == "TrackFrameNum")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(FString::FromInt(PolyMeshData->PolyMesh->GetNumberOfSamples())))
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:
	ECheckBoxState ShouldImportEnabled() const
	{
		return PolyMeshData->PolyMesh->bShouldImport ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnChangeShouldImport(ECheckBoxState NewState)
	{
		PolyMeshData->PolyMesh->bShouldImport = (NewState == ECheckBoxState::Checked);
	}

private:
	FPolyMeshDataPtr PolyMeshData;
	FText InformationText;
};
