// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AlembicLibrary : ModuleRules
{
    public AlembicLibrary(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
                "UnrealEd",
                "GeometryCache",
                "Imath",
                "AlembicLib",
                "MeshUtilities",
                "MaterialUtilities",
                "PropertyEditor",
                "SlateCore",
                "Slate",
                "Eigen",
                "RenderCore",
                "RHI"
			}
		);

        PrivateDependencyModuleNames.AddRange(
            new string[] {
				"EditorFramework",
                "MeshDescription",
				"StaticMeshDescription",
            }
        );

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
