// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcAssetImportData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbcAssetImportData)

UAbcAssetImportData::UAbcAssetImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

