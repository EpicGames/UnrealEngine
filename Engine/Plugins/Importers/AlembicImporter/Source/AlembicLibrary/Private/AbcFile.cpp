// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcFile.h"
#include "AbcImportSettings.h"
#include "AssetRegistry/AssetData.h"
#include "Async/ParallelFor.h"
#include "Materials/Material.h"
#include "MeshUtilities.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#include "AbcImporter.h"
#include "AbcTransform.h"
#include "AbcPolyMesh.h"

#include "AbcImportUtilities.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Logging/TokenizedMessage.h"
#include "AbcImportLogger.h"
#include "Misc/ScopedSlowTask.h"



#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <Alembic/Abc/ArchiveInfo.h>
#include "Materials/MaterialInstance.h"
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif


#define LOCTEXT_NAMESPACE "AbcFile"

const FString FAbcFile::NoFaceSetNameStr(TEXT("NoFaceSetName"));

FAbcFile::FAbcFile(const FString& InFilePath)
	: FilePath(InFilePath)
	, RootObject(nullptr)
	, MinFrameIndex(TNumericLimits<int32>::Max())
	, MaxFrameIndex(TNumericLimits<int32>::Min())
	, ArchiveSecondsPerFrame(0.0)
	, NumFrames(0)	
	, FramesPerSecond(0)
	, SecondsPerFrame(0.0)
	, StartFrameIndex(0)
	, EndFrameIndex(0)
	, ArchiveBounds(EForceInit::ForceInitToZero)
	, MinTime(TNumericLimits<float>::Max())
	, MaxTime(TNumericLimits<float>::Lowest())
	, ImportTimeOffset(0.0f)
	, ImportLength(0.0f)
{
}

FAbcFile::~FAbcFile()
{
	for (FAbcPolyMesh* PolyMesh : PolyMeshes)
	{
		delete PolyMesh;
	}
	PolyMeshes.Empty();

	for (FAbcTransform* Transform : Transforms)
	{
		delete Transform;
	}
	Transforms.Empty();
}

EAbcImportError FAbcFile::Open()
{
	Factory.setPolicy(Alembic::Abc::ErrorHandler::kQuietNoopPolicy);
	Factory.setOgawaNumStreams(12);
	
	// Extract Archive and compression type from file
	Archive = Factory.getArchive(TCHAR_TO_ANSI(*FPaths::ConvertRelativePathToFull(FilePath)), CompressionType);
	if (!Archive.valid())
	{
		TSharedPtr<FTokenizedMessage> Message;
		if (CompressionType == Alembic::AbcCoreFactory::IFactory::CoreType::kHDF5)
		{
			Message = FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("AbcHDF5NotSupported", "HDF5 Alembic files are not supported. Please convert the file to Ogawa format before importing."));
		}
		else
		{
			Message = FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("AbcUnknownFormat", "Unknown file format: not a valid Alembic."));
		}
		FAbcImportLogger::AddImportMessage(Message.ToSharedRef());
		return EAbcImportError::AbcImportError_InvalidArchive;
	}

	// Get Top/root object
	TopObject = Alembic::Abc::IObject(Archive, Alembic::Abc::kTop);
	if (!TopObject.valid())
	{
		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("AbcInvalidRoot", "Invalid root node: cannot proceed with import."));
		FAbcImportLogger::AddImportMessage(Message);
		return EAbcImportError::AbcImportError_NoValidTopObject;
	}

	TraverseAbcHierarchy(TopObject, nullptr);

	// Fallback values for 0/1 frame Alembic
	if (NumFrames < 2)
	{
		MinTime = 0.f;
		MaxTime = 0.f;
		MinFrameIndex = 0;
		MaxFrameIndex = NumFrames;
	}

	Alembic::Abc::ObjectHeader Header = TopObject.getHeader();

	// Determine top level archive bounding box
	const Alembic::Abc::MetaData ObjectMetaData = TopObject.getMetaData();
	Alembic::Abc::ICompoundProperty Properties = TopObject.getProperties();

	std::string TmpAppName;
	std::string TmpLibVersionString;
	std::string TmpDateWritten;
	std::string TmpUserDescription;

	GetArchiveInfo(Archive, TmpAppName, TmpLibVersionString, LibVersion, TmpDateWritten, TmpUserDescription);

	AppName = ANSI_TO_TCHAR(TmpAppName.c_str());
	LibVersionString = ANSI_TO_TCHAR(TmpLibVersionString.c_str());
	DateWritten = ANSI_TO_TCHAR(TmpDateWritten.c_str());
	UserDescription = ANSI_TO_TCHAR(TmpUserDescription.c_str());

	Alembic::Util::Digest ChildrenDigest;
	if (TopObject.getChildrenHash(ChildrenDigest))
	{
		Hash = ANSI_TO_TCHAR(ChildrenDigest.str().c_str());
	}
	else
	{
		// Fallback without actually hashing the file
		Hash = FString::Printf(TEXT("%s;%s"), *FPaths::GetBaseFilename(FilePath), *DateWritten);
	}

	Alembic::Abc::IBox3dProperty ArchiveBoundsProperty = Alembic::AbcGeom::GetIArchiveBounds(Archive, Alembic::Abc::ErrorHandler::kQuietNoopPolicy);

	if (ArchiveBoundsProperty.valid())
	{
		ArchiveBounds = AbcImporterUtilities::ExtractBounds(ArchiveBoundsProperty);
	}

	const int32 TimeSamplingIndex = Archive.getNumTimeSamplings() > 1 ? 1 : 0;

	Alembic::Abc::TimeSamplingPtr TimeSampler = Archive.getTimeSampling(TimeSamplingIndex);
	if (TimeSampler)
	{
		ArchiveSecondsPerFrame = static_cast<float>(TimeSampler->getTimeSamplingType().getTimePerCycle());
	}

	MeshUtilities = FModuleManager::Get().LoadModulePtr<IMeshUtilities>("MeshUtilities");
	
	return EAbcImportError::AbcImportError_NoError;
}

TArray<FAbcFile::FMetaData> FAbcFile::GetArchiveMetaData() const
{
	using FMetaDataEntry = TPairInitializer<FString, FString>;

	TArray<FMetaData> MetaData
	({
		FMetaDataEntry(TEXT("Abc.AppName"), AppName),
		FMetaDataEntry(TEXT("Abc.LibraryVersion"), LibVersionString),
		FMetaDataEntry(TEXT("Abc.WrittenOn"), DateWritten),
		FMetaDataEntry(TEXT("Abc.UserDescription"), UserDescription),
		FMetaDataEntry(TEXT("Abc.Hash"), Hash)
	});

	for (const FMetaData& CustomAttribute : CustomAttributes)
	{
		MetaData.Add(CustomAttribute);
	}

	return MetaData;
}

EAbcImportError FAbcFile::Import(UAbcImportSettings* InImportSettings)
{
	ImportSettings = InImportSettings;	

	FAbcSamplingSettings& SamplingSettings = ImportSettings->SamplingSettings;

	// Compute start/end frames based on the settings and report back the computed values to the settings for display and serialization
	StartFrameIndex = SamplingSettings.bSkipEmpty ? (FMath::Max(SamplingSettings.FrameStart, MinFrameIndex)) : SamplingSettings.FrameStart;
	SamplingSettings.FrameStart = StartFrameIndex;

	int32 LowerFrameIndex = FMath::Min((StartFrameIndex + 1), MaxFrameIndex);
	int32 UpperFrameIndex = FMath::Max((StartFrameIndex + 1), MaxFrameIndex);
	EndFrameIndex = SamplingSettings.FrameEnd == 0 ? MaxFrameIndex : FMath::Clamp(SamplingSettings.FrameEnd, LowerFrameIndex, UpperFrameIndex);
	SamplingSettings.FrameEnd = EndFrameIndex;

	if (ImportSettings->ImportType == EAlembicImportType::StaticMesh)
	{
		EndFrameIndex = StartFrameIndex;
	}

	int32 FrameSpan = EndFrameIndex - StartFrameIndex;
	// If Start==End or Start > End output error message due to invalid frame span
	if (FrameSpan <= 0 && ImportSettings->ImportType != EAlembicImportType::StaticMesh)
	{
		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("NoFramesForMeshObject", "Invalid frame range specified {0} - {1}."), FText::FromString(FString::FromInt(StartFrameIndex)), FText::FromString(FString::FromInt(EndFrameIndex))));
		FAbcImportLogger::AddImportMessage(Message);
		return AbcImportError_FailedToImportData;
	}

	// Calculate time step and min/max frame indices according to user sampling settings
	float TimeStep = 0.0f;
	float CacheLength = MaxTime - MinTime;
	EAlembicSamplingType SamplingType = SamplingSettings.SamplingType;
	switch (SamplingType)
	{
	case EAlembicSamplingType::PerFrame:
	{
		// Calculates the time step required to get the number of frames
		TimeStep = !FMath::IsNearlyZero(ArchiveSecondsPerFrame) ? ArchiveSecondsPerFrame : CacheLength / (float)(MaxFrameIndex - MinFrameIndex);
		break;
	}

	case EAlembicSamplingType::PerTimeStep:
	{
		// Calculates the original time step and the ratio between it and the user specified time step
		const float OriginalTimeStep = CacheLength / (float)(MaxFrameIndex - MinFrameIndex);
		const float FrameStepRatio = OriginalTimeStep / SamplingSettings.TimeSteps;
		TimeStep = SamplingSettings.TimeSteps;

		AbcImporterUtilities::CalculateNewStartAndEndFrameIndices(FrameStepRatio, StartFrameIndex, EndFrameIndex);
		FrameSpan = EndFrameIndex - StartFrameIndex;
		break;
	}
	case EAlembicSamplingType::PerXFrames:
	{
		// Calculates the original time step and the ratio between it and the user specified time step
		const float OriginalTimeStep = CacheLength / (float)(MaxFrameIndex - MinFrameIndex);
		const float FrameStepRatio = OriginalTimeStep / ((float)SamplingSettings.FrameSteps * OriginalTimeStep);
		TimeStep = ((float)SamplingSettings.FrameSteps * OriginalTimeStep);

		AbcImporterUtilities::CalculateNewStartAndEndFrameIndices(FrameStepRatio, StartFrameIndex, EndFrameIndex);
		FrameSpan = EndFrameIndex - StartFrameIndex;
		break;
	}

	default:
		checkf(false, TEXT("Incorrect sampling type found in import settings (%i)"), (uint8)SamplingType);
	}

	SecondsPerFrame = TimeStep;
	FramesPerSecond = TimeStep > 0.f ? (1.f / TimeStep) : 30.f;
	ImportLength = static_cast<float>(FrameSpan) * TimeStep;

	// Calculate time offset from start of import animation range
	ImportTimeOffset = static_cast<float>(StartFrameIndex) / FramesPerSecond;

	// Read first-frames for both the transforms and poly meshes

	bool bValidFirstFrames = true;
	for (FAbcTransform* Transform : Transforms)
	{
		bValidFirstFrames &= Transform->ReadFirstFrame(static_cast<float>(StartFrameIndex) / FramesPerSecond, StartFrameIndex);
	}

	for (FAbcPolyMesh* PolyMesh : PolyMeshes)
	{
		if (PolyMesh->bShouldImport)
		{
			bValidFirstFrames &= PolyMesh->ReadFirstFrame(static_cast<float>(StartFrameIndex) / FramesPerSecond, StartFrameIndex);
		}
	}	

	if (!bValidFirstFrames)
	{
		return AbcImportError_FailedToImportData;
	}

	// Add up all poly mesh bounds
	FBoxSphereBounds::Builder MeshBoundsBuilder;
	for (const FAbcPolyMesh* PolyMesh : PolyMeshes)
	{
		if (PolyMesh->bShouldImport)
		{
			MeshBoundsBuilder += PolyMesh->SelfBounds;
			MeshBoundsBuilder += PolyMesh->ChildBounds;
		}
	}
	FBoxSphereBounds MeshBounds(MeshBoundsBuilder);

	// If there were not bounds available at the archive level or the mesh bounds are larger than the archive bounds use those
	if (FMath::IsNearlyZero(ArchiveBounds.SphereRadius) || MeshBounds.SphereRadius >ArchiveBounds.SphereRadius )
	{
		ArchiveBounds = MeshBounds;
	}
	AbcImporterUtilities::ApplyConversion(ArchiveBounds, ImportSettings->ConversionSettings);

	// If the users opted to try and find materials in the project whos names match one of the face sets
	if (ImportSettings->MaterialSettings.bFindMaterials)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		TArray<FAssetData> AssetData;
		const UClass* Class = UMaterialInterface::StaticClass();
		AssetRegistryModule.Get().GetAssetsByClass(Class->GetClassPathName(), AssetData, true);
		for (FAbcPolyMesh* PolyMesh : PolyMeshes)
		{
			for (const FString& FaceSetName : PolyMesh->FaceSetNames)
			{
				UMaterialInterface** ExistingMaterial = MaterialMap.Find(*FaceSetName);
				if (!ExistingMaterial)
				{
					FAssetData* MaterialAsset = AssetData.FindByPredicate([=](const FAssetData& Asset)
					{
						return Asset.AssetName.ToString() == FaceSetName;
					});

					if (MaterialAsset)
					{
						UMaterialInterface* FoundMaterialInterface = Cast<UMaterialInterface>(MaterialAsset->GetAsset());
						if (FoundMaterialInterface)
						{							
							MaterialMap.Add(FaceSetName, FoundMaterialInterface);

							if (ImportSettings->ImportType == EAlembicImportType::Skeletal)
							{
								FoundMaterialInterface->SetMaterialUsage(MATUSAGE_SkeletalMesh);
								FoundMaterialInterface->SetMaterialUsage(MATUSAGE_MorphTargets);
							}
							else if (ImportSettings->ImportType == EAlembicImportType::GeometryCache)
							{
								FoundMaterialInterface->SetMaterialUsage(MATUSAGE_GeometryCache);
							}
						}
					}
					else
					{
						TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("NoMaterialForFaceSet", "Unable to find matching Material for Face Set {0}, using default material instead."), FText::FromString(FaceSetName)));
						FAbcImportLogger::AddImportMessage(Message);
					}
				}
			}
		}
	}
	// Or the user opted to create materials with and for the faceset names in this ABC file
	else if (ImportSettings->MaterialSettings.bCreateMaterials)
	{
		// Creates materials according to the face set names that were found in the Alembic file
		for (FAbcPolyMesh* PolyMesh : PolyMeshes)
		{
			for (const FString& FaceSetName : PolyMesh->FaceSetNames)
			{
				// Preventing duplicate material creation
				UMaterialInterface** ExistingMaterial = MaterialMap.Find(*FaceSetName);
				if (!ExistingMaterial)
				{ 
					UMaterial* Material = NewObject<UMaterial>((UObject*)GetTransientPackage(), *FaceSetName);
					Material->SetUsageByFlag(MATUSAGE_MorphTargets, true);
					MaterialMap.Add(FaceSetName, Material);
				}
			}
		}
	}

	// Count total number of facesets (1 or more per mesh)
	int32 NumFacesets = 0;
	for (const FAbcPolyMesh* PolyMesh : PolyMeshes)
	{
		if (PolyMesh->bShouldImport)
		{
			check(!PolyMesh->FaceSetNames.IsEmpty());
			NumFacesets += PolyMesh->FaceSetNames.Num();
		}
	}
	
	if (NumFacesets == 0)
	{
		return AbcImportError_NoMeshes;
	}

	// Populate the list of unique face set names from the meshes that should be imported regardless of the import material settings
	// and construct mapping from faceset index (in flat list of imported mesh facesets) to material slot
	LookupMaterialSlot.SetNum(NumFacesets);

	int32 FacesetMaterialIndex = 0;
	for (const FAbcPolyMesh* PolyMesh : PolyMeshes)
	{
		if (PolyMesh->bShouldImport)
		{
			for (const FString& FaceSetName : PolyMesh->FaceSetNames)
			{
				int32 MaterialSlotIndex = UniqueFaceSetNames.Find(FaceSetName);
				if (MaterialSlotIndex == INDEX_NONE)
				{
					MaterialSlotIndex = UniqueFaceSetNames.Num();
					UniqueFaceSetNames.Add(FaceSetName);
				}

				LookupMaterialSlot[FacesetMaterialIndex] = MaterialSlotIndex;
				FacesetMaterialIndex++;
			}
		}
	}

	check(FacesetMaterialIndex == NumFacesets);

	return AbcImportError_NoError;
}

void FAbcFile::TraverseAbcHierarchy(const Alembic::Abc::IObject& InObject, IAbcObject* InParent)
{
	if (!InObject)
	{
		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("AbcInvalidObject", "Invalid object detected: the Alembic file may be corrupted."));
		FAbcImportLogger::AddImportMessage(Message);
		return;
	}

	// Get Header and MetaData info from current Alembic Object
	Alembic::AbcCoreAbstract::ObjectHeader Header = InObject.getHeader();
	const Alembic::Abc::MetaData ObjectMetaData = InObject.getMetaData();
	const uint32 NumChildren = InObject.getNumChildren();

	IAbcObject* CreatedObject = nullptr;

	if (AbcImporterUtilities::IsType<Alembic::AbcGeom::IPolyMesh>(ObjectMetaData))
	{
		Alembic::AbcGeom::IPolyMesh Mesh = Alembic::AbcGeom::IPolyMesh(InObject, Alembic::Abc::kWrapExisting);
		
		FAbcPolyMesh* PolyMesh = new FAbcPolyMesh(Mesh, this, InParent);
		PolyMeshes.Add(PolyMesh);
		CreatedObject = PolyMesh;
		Objects.Add(CreatedObject);

		ExtractCustomAttributes(Mesh);

		// Ignore constant nodes for the computation of the animation time/index range
		// Note that a constant mesh could be animated through its parent transform
		// in which case, the animation range will reflect that of the IXform
		if (PolyMesh->GetNumberOfSamples() > 1)
		{
			MinTime = FMath::Min(MinTime, PolyMesh->GetTimeForFirstData());
			MaxTime = FMath::Max(MaxTime, PolyMesh->GetTimeForLastData());
			NumFrames = FMath::Max(NumFrames, PolyMesh->GetNumberOfSamples());
			MinFrameIndex = FMath::Min(MinFrameIndex, PolyMesh->GetFrameIndexForFirstData());
			MaxFrameIndex = FMath::Max(MaxFrameIndex, PolyMesh->GetFrameIndexForFirstData() + PolyMesh->GetNumberOfSamples());
		}
	}
	else if (AbcImporterUtilities::IsType<Alembic::AbcGeom::IXform>(ObjectMetaData))
	{
		Alembic::AbcGeom::IXform Xform = Alembic::AbcGeom::IXform(InObject, Alembic::Abc::kWrapExisting);
		FAbcTransform* Transform = new FAbcTransform(Xform, this, InParent);
		Transforms.Add(Transform);
		CreatedObject = Transform;
		Objects.Add(CreatedObject);

		// Ignore constant nodes for the computation of the animation time/index range
		// A constant identity transform has 0 frame while a constant non-identity transform has 1 frame
		// In either case, the min/max times are invalid and irrelevant
		if (Transform->GetNumberOfSamples() > 1)
		{
			MinTime = FMath::Min(MinTime, Transform->GetTimeForFirstData());
			MaxTime = FMath::Max(MaxTime, Transform->GetTimeForLastData());
			NumFrames = FMath::Max(NumFrames, Transform->GetNumberOfSamples());
			MinFrameIndex = FMath::Min(MinFrameIndex, Transform->GetFrameIndexForFirstData());
			MaxFrameIndex = FMath::Max(MaxFrameIndex, Transform->GetFrameIndexForFirstData() + Transform->GetNumberOfSamples());
		}
	}

	// Check for animated visibility property
	{
		const Alembic::Abc::ICompoundProperty CompoundProperty = InObject.getProperties();

		if (CompoundProperty.getPropertyHeader(Alembic::AbcGeom::kVisibilityPropertyName))
		{
			const Alembic::AbcGeom::IVisibilityProperty VisibilityProperty = Alembic::AbcGeom::IVisibilityProperty(CompoundProperty, Alembic::AbcGeom::kVisibilityPropertyName);

			int32 NumSamplesVisibility = VisibilityProperty.getNumSamples();

			if (NumSamplesVisibility > 1)
			{
				const Alembic::AbcCoreAbstract::TimeSamplingPtr TimeSampler = VisibilityProperty.getTimeSampling();
				const Alembic::AbcCoreAbstract::TimeSamplingType SamplingType = TimeSampler->getTimeSamplingType();

				const float StartTimeVisibility = (float)TimeSampler->getSampleTime(0);
				const int32 MinFrameIndexVisibility = FMath::CeilToInt(StartTimeVisibility / (float)SamplingType.getTimePerCycle());
				const int32 MaxFrameIndexVisibility = MinFrameIndexVisibility + NumSamplesVisibility - 1;

				MinTime = FMath::Min(MinTime, StartTimeVisibility);
				MaxTime = FMath::Max(MaxTime, (float)TimeSampler->getSampleTime(NumSamplesVisibility - 1));
				NumFrames = FMath::Max(NumFrames, NumSamplesVisibility);
				MinFrameIndex = FMath::Min(MinFrameIndex, MinFrameIndexVisibility);
				MaxFrameIndex = FMath::Max(MaxFrameIndex, MaxFrameIndexVisibility);
			}
		}
	}

	if (RootObject == nullptr && CreatedObject != nullptr)
	{
		RootObject = CreatedObject;
	}

	// Recursive traversal of child objects
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		const Alembic::Abc::IObject& AbcChildObject = InObject.getChild(ChildIndex);
		TraverseAbcHierarchy(AbcChildObject, CreatedObject);
	}
}

void FAbcFile::ExtractCustomAttributes(const Alembic::AbcGeom::IPolyMesh& InMesh)
{
	// Extract the custom attributes from the mesh's arbitrary GeomParams
	Alembic::AbcGeom::ICompoundProperty ArbParams = InMesh.getSchema().getArbGeomParams();
	if (ArbParams)
	{
		FString ObjectName(ANSI_TO_TCHAR(InMesh.getName().c_str()));

		for (int Index = 0; Index < ArbParams.getNumProperties(); ++Index)
		{
			Alembic::Abc::PropertyHeader PropertyHeader = ArbParams.getPropertyHeader(Index);
			Alembic::Abc::PropertyType PropType = PropertyHeader.getPropertyType();
			Alembic::Abc::DataType DataType = PropertyHeader.getDataType();

			// Extract only scalar string attributes
			if (PropType == Alembic::Abc::kScalarProperty && DataType.getPod() == Alembic::Util::kStringPOD)
			{
				std::string PropName = PropertyHeader.getName();
				Alembic::Abc::IStringProperty Param(ArbParams, PropName);
				FString AttributeName(ANSI_TO_TCHAR(PropName.c_str()));
				FString AttributeValue(ANSI_TO_TCHAR(Param.getValue().c_str()));

				AttributeName = FString::Printf(TEXT("Abc.%s.%s"), *ObjectName, *AttributeName);
				CustomAttributes.Add(AttributeName, AttributeValue);
			}
		}
	}
}

void FAbcFile::ReadFrame(int32 FrameIndex, const EFrameReadFlags InFlags, const int32 ReadIndex /*= INDEX_NONE*/)
{
	for (IAbcObject* Object : Objects)
	{
		Object->SetFrameAndTime(static_cast<float>(FrameIndex) / FramesPerSecond, FrameIndex, InFlags, ReadIndex);
	}
}

void FAbcFile::CleanupFrameData(const int32 ReadIndex)
{
	for (IAbcObject* Object : Objects)
	{
		Object->PurgeFrameData(ReadIndex);
	}
}

bool FAbcFile::ProcessFrames(TFunctionRef<void(int32, FAbcFile*)> InCallback, const EFrameReadFlags InFlags, const FSlowTask* SlowTask)
{
	const int32 NumWorkerThreads = FMath::Clamp(FTaskGraphInterface::Get().GetNumWorkerThreads(), 1, MaxNumberOfResidentSamples);
	const bool bSingleThreaded = ImportSettings->NumThreads == 1 ||
								 EnumHasAnyFlags(InFlags, EFrameReadFlags::ForceSingleThreaded) || !FApp::ShouldUseThreadingForPerformance();

	volatile bool bIsCancelled = false;

	if (bSingleThreaded)
	{
		for (int32 FrameIndex = StartFrameIndex; FrameIndex <= EndFrameIndex; ++FrameIndex)
		{
			if (SlowTask && SlowTask->ShouldCancel())
			{
				return false;
			}

			ReadFrame(FrameIndex, InFlags, 0);
			InCallback(FrameIndex, this);
			CleanupFrameData(0);
		}
	}
	else
	{
		// Frame data can be read concurrently but will be processed sequentially.
		std::atomic<int32> WriteFrameIndex = StartFrameIndex;
		FCriticalSection Mutex;
		FEvent* FrameWrittenEvent = FPlatformProcess::GetSynchEventFromPool();
		if (!FrameWrittenEvent)
		{
			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("NoSynchEvent", "Unable to get synchronization event for parallelized Alembic frame data import."));
			FAbcImportLogger::AddImportMessage(Message);
			return false;
		}

		ParallelFor(NumWorkerThreads, [this, InFlags, InCallback, bSingleThreaded, NumWorkerThreads, &WriteFrameIndex, &Mutex, &FrameWrittenEvent, &bIsCancelled, SlowTask](int32 ThreadIndex)
		{
			int32 FrameIndex = StartFrameIndex + ThreadIndex;

			while (FrameIndex <= EndFrameIndex && !bIsCancelled)
			{
				if (IsInGameThread() && SlowTask && SlowTask->ShouldCancel())
				{
					bIsCancelled = true;
					break;
				}

				// Read frame data into memory
				ReadFrame(FrameIndex, InFlags, ThreadIndex);

				// Wait until it's our turn to process this frame.
				while (WriteFrameIndex < FrameIndex)
				{
					FrameWrittenEvent->Wait(100);
				}

				{
					FScopeLock WriteLock(&Mutex);

					// Call the user defined callback.
					InCallback(FrameIndex, this);
					
					// Mark the next frame index as ready for processing.
					++WriteFrameIndex;

					FrameWrittenEvent->Trigger();
				}

				// Now cleanup the frame data
				CleanupFrameData(ThreadIndex);

				// Get new frame index to read for next run cycle
				FrameIndex += NumWorkerThreads;
			};
		});

		FPlatformProcess::ReturnSynchEventToPool(FrameWrittenEvent);
	}

	return !bIsCancelled;
}

const int32 FAbcFile::GetMinFrameIndex() const
{
	return MinFrameIndex;
}

const int32 FAbcFile::GetMaxFrameIndex() const
{
	return MaxFrameIndex;
}

const int32 FAbcFile::GetStartFrameIndex() const
{
	return StartFrameIndex;
}

const int32 FAbcFile::GetEndFrameIndex() const
{
	return EndFrameIndex;
}

const UAbcImportSettings* FAbcFile::GetImportSettings() const 
{
	return ImportSettings;
}

const TArray<FAbcPolyMesh*>& FAbcFile::GetPolyMeshes() const
{
	return PolyMeshes;
}

const TArray<FAbcTransform*>& FAbcFile::GetTransforms() const
{
	return Transforms;
}

const int32 FAbcFile::GetNumPolyMeshes() const
{
	return PolyMeshes.Num();
}

const FString FAbcFile::GetFilePath() const
{
	return FilePath;
}

const float FAbcFile::GetImportTimeOffset() const
{
	return ImportTimeOffset;
}

const float FAbcFile::GetImportLength() const
{
	return ImportLength;
}

const int32 FAbcFile::GetImportNumFrames() const
{
	return EndFrameIndex - StartFrameIndex;
}

const float FAbcFile::GetFramerate() const
{
	return FramesPerSecond;
}

const float FAbcFile::GetSecondsPerFrame() const
{
	return SecondsPerFrame;
}

int32 FAbcFile::GetFrameIndex(float Time)
{
	if (SecondsPerFrame > 0.f)
	{
		int32 FrameIndex = StartFrameIndex + FMath::FloorToInt(Time / SecondsPerFrame);
		return FMath::Clamp(FrameIndex, StartFrameIndex, EndFrameIndex);
	}
	return 0;
}

const FBoxSphereBounds& FAbcFile::GetArchiveBounds() const
{
	return ArchiveBounds;
}

const bool FAbcFile::ContainsHeterogeneousMeshes() const
{
	bool bHomogeneous = true;

	for (const FAbcPolyMesh* Mesh : PolyMeshes)
	{
		bHomogeneous &= ( Mesh->bConstantTopology || !Mesh->bShouldImport );
	}

	return !bHomogeneous;
}

IMeshUtilities* FAbcFile::GetMeshUtilities() const
{
	return MeshUtilities;
}

UMaterialInterface** FAbcFile::GetMaterialByName(const FString& InMaterialName)
{
	return MaterialMap.Find(InMaterialName);
}

#undef LOCTEXT_NAMESPACE // "AbcFile"
