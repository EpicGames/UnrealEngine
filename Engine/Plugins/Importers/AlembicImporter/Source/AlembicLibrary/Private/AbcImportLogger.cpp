// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcImportLogger.h"
#include "Logging/MessageLog.h"

TArray<TSharedRef<FTokenizedMessage>> FAbcImportLogger::TokenizedErrorMessages;
FCriticalSection FAbcImportLogger::MessageLock;

void FAbcImportLogger::AddImportMessage(const TSharedRef<FTokenizedMessage> Message)
{
	MessageLock.Lock();
	TokenizedErrorMessages.Add(Message);
	MessageLock.Unlock();
}

void FAbcImportLogger::OutputMessages(const FString& PageName)
{	
	static const FName LogName = "AssetTools";
	FMessageLog MessageLog(LogName);
	MessageLog.NewPage(FText::FromString(PageName));
	
	MessageLock.Lock();	
	MessageLog.AddMessages(TokenizedErrorMessages);
	TokenizedErrorMessages.Empty();
	MessageLock.Unlock();

	MessageLog.Open();
}

TArray<TSharedRef<FTokenizedMessage>> FAbcImportLogger::RetrieveMessages()
{	
	return MoveTemp(TokenizedErrorMessages);
}
