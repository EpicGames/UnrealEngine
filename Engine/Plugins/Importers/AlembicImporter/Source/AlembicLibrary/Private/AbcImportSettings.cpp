// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcImportSettings.h"
#include "UObject/Package.h"
#include "UObject/ReleaseObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbcImportSettings)

UAbcImportSettings::UAbcImportSettings(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	ImportType = EAlembicImportType::StaticMesh;
	bReimport = false;
}

UAbcImportSettings* UAbcImportSettings::Get()
{	
	static UAbcImportSettings* DefaultSettings = nullptr;
	if (!DefaultSettings)
	{
		// This is a singleton, use default object
		DefaultSettings = DuplicateObject(GetMutableDefault<UAbcImportSettings>(), GetTransientPackage());
		DefaultSettings->AddToRoot();
	}
	
	return DefaultSettings;
}

void UAbcImportSettings::Serialize(FArchive& Archive)
{
	Super::Serialize( Archive );

	Archive.UsingCustomVersion(FReleaseObjectVersion::GUID);

	if (Archive.IsLoading() && Archive.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::AbcVelocitiesSupport)
	{
		if (GeometryCacheSettings.bCalculateMotionVectorsDuringImport_DEPRECATED)
		{
			GeometryCacheSettings.MotionVectors = EAbcGeometryCacheMotionVectorsImport::CalculateMotionVectorsDuringImport;
		}
	}
}
