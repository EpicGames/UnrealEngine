// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcImportSettingsCustomization.h"

#include "AbcImportSettings.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "PropertyRestriction.h"
#include "AssetToolsModule.h"
#include "Misc/NamePermissionList.h"

void FAbcImportSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
{
	TSharedRef<IPropertyHandle> ImportType = LayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAbcImportSettings, ImportType));
	
	uint8 EnumValue;
	ImportType->GetValue(EnumValue);
	IDetailCategoryBuilder& CompressionBuilder = LayoutBuilder.EditCategory("Compression");
	CompressionBuilder.SetCategoryVisibility(EnumValue == (uint8)EAlembicImportType::Skeletal);

	IDetailCategoryBuilder& StaticMeshBuilder = LayoutBuilder.EditCategory("StaticMesh");
	StaticMeshBuilder.SetCategoryVisibility(EnumValue == (uint8)EAlembicImportType::StaticMesh);

	IDetailCategoryBuilder& GeomCacheBuilder = LayoutBuilder.EditCategory("GeometryCache");
	GeomCacheBuilder.SetCategoryVisibility(EnumValue == (uint8)EAlembicImportType::GeometryCache);

	FSimpleDelegate OnImportTypeChangedDelegate = FSimpleDelegate::CreateSP(this, &FAbcImportSettingsCustomization::OnImportTypeChanged, &LayoutBuilder);
	ImportType->SetOnPropertyValueChanged(OnImportTypeChangedDelegate);

	TArray<TWeakObjectPtr<UObject>> Objects;
	LayoutBuilder.GetObjectsBeingCustomized(Objects);

	
	TWeakObjectPtr<UObject>* SettingsObject = Objects.FindByPredicate([](const TWeakObjectPtr<UObject>& Object) { return Object->IsA<UAbcImportSettings>(); } );

	UAbcImportSettings* CurrentSettings = SettingsObject ? Cast<UAbcImportSettings>(SettingsObject->Get()) : nullptr;

	if (CurrentSettings)
	{
		if (CurrentSettings->bReimport)
		{
			UEnum* ImportTypeEnum = StaticEnum<EAlembicImportType>();
			static FText RestrictReason = NSLOCTEXT("AlembicImportFactory", "ReimportRestriction", "Unable to change type while reimporting");
			TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShareable(new FPropertyRestriction(RestrictReason));

			for (uint8 EnumIndex = 0; EnumIndex < (ImportTypeEnum->GetMaxEnumValue() + 1); ++EnumIndex)
			{
				if (EnumValue != EnumIndex)
				{
					EnumRestriction->AddDisabledValue(ImportTypeEnum->GetNameStringByIndex(EnumIndex));
				}
			}
			ImportType->AddRestriction(EnumRestriction.ToSharedRef());
		}
		else
		{
			static bool bGeometryCacheAllowed = []() -> bool
			{
				IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
				TSharedPtr<FPathPermissionList> AssetClassPermissionList = AssetTools.GetAssetClassPathPermissionList(EAssetClassAction::ImportAsset);
				if (AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
				{
					return AssetClassPermissionList->PassesFilter(TEXT("/Script/GeometryCache.GeometryCache"));
				}

				return true;
			}();

			UEnum* ImportTypeEnum = StaticEnum<EAlembicImportType>();
			static FText RestrictReason = NSLOCTEXT("AlembicImportFactory", "ImportRestriction", "Geometry Cache import disabled");
			TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShareable(new FPropertyRestriction(RestrictReason));

			if (!bGeometryCacheAllowed)
			{
				EnumRestriction->AddHiddenValue(ImportTypeEnum->GetNameStringByValue((uint8)EAlembicImportType::GeometryCache));
				ImportType->AddRestriction(EnumRestriction.ToSharedRef());
			}
		}
	}	
}

TSharedRef<IDetailCustomization> FAbcImportSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FAbcImportSettingsCustomization());
}

void FAbcImportSettingsCustomization::OnImportTypeChanged(IDetailLayoutBuilder* LayoutBuilder)
{
	LayoutBuilder->ForceRefreshDetails();
}

TSharedRef<IPropertyTypeCustomization> FAbcSamplingSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FAbcSamplingSettingsCustomization);
}

void FAbcSamplingSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();

		// Skip properties that are already marked hidden
		if (ChildHandle->IsCustomized())
		{
			continue;
		}

		if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FAbcSamplingSettings, SamplingType))
		{
			SamplingTypeHandle = ChildHandle; 
		}

		IDetailPropertyRow& Property = StructBuilder.AddProperty(ChildHandle);
		static const FName EditConditionName = "EnumCondition";
		int32 EnumCondition = ChildHandle->GetIntMetaData(EditConditionName);
		Property.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FAbcSamplingSettingsCustomization::ArePropertiesVisible, EnumCondition)));
	}
}

EVisibility FAbcSamplingSettingsCustomization::ArePropertiesVisible(const int32 VisibleType) const
{
	uint8 Value = 0;
	if (SamplingTypeHandle.IsValid() && (SamplingTypeHandle->GetValue(Value) == FPropertyAccess::Success))
	{
		return (Value == (uint8)VisibleType || VisibleType == 0) ? EVisibility::Visible : EVisibility::Collapsed;
	}
	
	return EVisibility::Visible;
}

TSharedRef<IPropertyTypeCustomization> FAbcCompressionSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FAbcCompressionSettingsCustomization);
}

void FAbcCompressionSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();

		if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FAbcCompressionSettings, BaseCalculationType))
		{
			BaseCalculationTypeHandle = ChildHandle;
		}

		IDetailPropertyRow& Property = StructBuilder.AddProperty(ChildHandle);
		static const FName EditConditionName = "EnumCondition";
		int32 EnumCondition = ChildHandle->GetIntMetaData(EditConditionName);
		Property.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FAbcCompressionSettingsCustomization::ArePropertiesVisible, EnumCondition)));
	}
}

EVisibility FAbcCompressionSettingsCustomization::ArePropertiesVisible(const int32 VisibleType) const
{
	uint8 Value = 0;
	if (BaseCalculationTypeHandle.IsValid() && (BaseCalculationTypeHandle->GetValue(Value) == FPropertyAccess::Success))
	{
		return (Value == (uint8)VisibleType || VisibleType == 0) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

TSharedRef<IPropertyTypeCustomization> FAbcConversionSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FAbcConversionSettingsCustomization);
}

void FAbcConversionSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FSimpleDelegate OnPresetChanged = FSimpleDelegate::CreateSP(this, &FAbcConversionSettingsCustomization::OnConversionPresetChanged);
	FSimpleDelegate OnValueChanged = FSimpleDelegate::CreateSP(this, &FAbcConversionSettingsCustomization::OnConversionValueChanged);

	TArray<void*> StructPtrs;
	StructPropertyHandle->AccessRawData(StructPtrs);
	if (StructPtrs.Num() == 1)
	{
		Settings = (FAbcConversionSettings*)StructPtrs[0];
	}

	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();

		if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FAbcConversionSettings, Preset))
		{			
			ChildHandle->SetOnPropertyValueChanged(OnPresetChanged);
		}
		else
		{
			ChildHandle->SetOnPropertyValueChanged(OnValueChanged);
			ChildHandle->SetOnChildPropertyValueChanged(OnValueChanged);			
		}

		IDetailPropertyRow& Property = StructBuilder.AddProperty(ChildHandle);
	}
}

void FAbcConversionSettingsCustomization::OnConversionPresetChanged()
{
	if (Settings)
	{
		// Set values to specified preset
		switch (Settings->Preset)
		{
			case EAbcConversionPreset::Max:
			{
				Settings->bFlipU = false;
				Settings->bFlipV = true;
				Settings->Scale = FVector(1.0f, -1.0f, 1.0f);
				Settings->Rotation = FVector::ZeroVector;
				break;
			}

			case EAbcConversionPreset::Maya:
			{
				Settings->bFlipU = false;
				Settings->bFlipV = true;
				Settings->Scale = FVector(1.0f, -1.0f, 1.0f);
				Settings->Rotation = FVector(90.0f, 0.0f, 0);
				break;
			}
		}
	}
}

void FAbcConversionSettingsCustomization::OnConversionValueChanged()
{
	// Set conversion preset to custom
	Settings->Preset = EAbcConversionPreset::Custom;
}
