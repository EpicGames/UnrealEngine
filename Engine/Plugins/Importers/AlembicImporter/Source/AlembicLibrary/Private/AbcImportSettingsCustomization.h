// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"

struct EVisibility;

class UAbcImportSettings;

class FAbcImportSettingsCustomization : public IDetailCustomization
{
public:
	/** Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder) override;
	/** End IDetailCustomization interface */
public:

	/** Creates a new instance. */
	static TSharedRef<IDetailCustomization> MakeInstance();
protected:
	void OnImportTypeChanged(IDetailLayoutBuilder* LayoutBuilder);

protected:
	UAbcImportSettings* ImportSettings;
};

/** AbcSampling settings customizations handles hiding/showing properties according to the frame sampling type */
class FAbcSamplingSettingsCustomization
	: public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FAbcSamplingSettingsCustomization()
	{ }

	virtual ~FAbcSamplingSettingsCustomization() = default;

	/** Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** End IPropertyTypeCustomization interface */

protected:
	EVisibility ArePropertiesVisible(const int32 VisibleType) const;
	TSharedPtr<IPropertyHandle> SamplingTypeHandle;
};

/** AbcCompression settings customizations handles hiding/showing properties according to the base calculation type */
class FAbcCompressionSettingsCustomization
	: public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FAbcCompressionSettingsCustomization()
	{ }

	virtual ~FAbcCompressionSettingsCustomization() = default;
	
	/** Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** End IPropertyTypeCustomization interface */

protected:
	EVisibility ArePropertiesVisible(const int32 VisibleType) const;
	TSharedPtr<IPropertyHandle> BaseCalculationTypeHandle;
};

class FAbcConversionSettingsCustomization
	: public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FAbcConversionSettingsCustomization() : Settings(nullptr) { }
	virtual ~FAbcConversionSettingsCustomization() = default;

	/** Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** End IPropertyTypeCustomization interface */

protected:
	void OnConversionPresetChanged();
	void OnConversionValueChanged();

	struct FAbcConversionSettings* Settings;
};
