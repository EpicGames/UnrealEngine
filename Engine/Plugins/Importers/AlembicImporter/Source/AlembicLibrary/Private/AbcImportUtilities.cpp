// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcImportUtilities.h"
#include "AbcImportLogger.h"
#include "Math/TransformCalculus2D.h"

#include "AbcFile.h"
#include "AbcImportSettings.h"
#include "AbcImporter.h"
#include "AbcPolyMesh.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/ParallelFor.h"
#include "GeometryCacheMeshData.h"
#include "Logging/TokenizedMessage.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MeshUtilities.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "PackageTools.h"
#include "RenderMath.h"
#include "UObject/Package.h"

#if PLATFORM_WINDOWS
#include "RenderMath.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <Alembic/AbcCoreFactory/All.h>
#include <Alembic/AbcCoreOgawa/All.h>
#include <Imath/ImathVec.h>
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "Rendering/SkeletalMeshLODModel.h"

#define LOCTEXT_NAMESPACE "AbcImporterUtilities"

FMatrix AbcImporterUtilities::ConvertAlembicMatrix(const Alembic::Abc::M44d& AbcMatrix)
{
	FMatrix Matrix;
	for (uint32 i = 0; i < 16; ++i)
	{
		Matrix.M[i >> 2][i % 4] = (float)AbcMatrix.getValue()[i];
	}

	return Matrix;
}

uint32 AbcImporterUtilities::GenerateMaterialIndicesFromFaceSets(Alembic::AbcGeom::IPolyMeshSchema &Schema, const Alembic::Abc::ISampleSelector FrameSelector, TArray<int32> &MaterialIndicesOut)
{
	// Retrieve face set names to determine if we will have to process face sets (used for face-material indices)
	std::vector<std::string> FaceSetNames;
	Schema.getFaceSetNames(FaceSetNames);

	if (FaceSetNames.size() == 0)
	{
		return 1; // We will create a default unnamed faceset
	}

	// Number of unique face sets found in the Alembic Object
	uint32 NumUniqueFaceSets = 0;
	
	// Loop over the face-set names
	for (uint32 FaceSetIndex = 0; FaceSetIndex < FaceSetNames.size(); ++FaceSetIndex)
	{
		const std::string& FaceSetName = FaceSetNames[FaceSetIndex];

		Alembic::AbcGeom::IFaceSet FaceSet = Schema.getFaceSet(FaceSetName);
		Alembic::AbcGeom::IFaceSetSchema FaceSetSchema = FaceSet.getSchema();
		Alembic::AbcGeom::IFaceSetSchema::Sample FaceSetSample;
		FaceSetSchema.get(FaceSetSample, FrameSelector);

		// Retrieve face indices that are part of this face set
		Alembic::Abc::Int32ArraySamplePtr Faces = FaceSetSample.getFaces();
		const bool bFacesAvailable = (Faces != nullptr);
		const int NumFaces = Faces->size();

		// Set the shared Material index for all the contained faces
		for (int32 i = 0; i < NumFaces && NumFaces < MaterialIndicesOut.Num(); ++i)
		{
			const int32 FaceIndex = Faces->get()[i];
			if (MaterialIndicesOut.IsValidIndex(FaceIndex))
			{
				MaterialIndicesOut[FaceIndex] = FaceSetIndex;
			}
		}

		// Found a new unique faceset
		NumUniqueFaceSets++;
	}

	return NumUniqueFaceSets;
}

void AbcImporterUtilities::RetrieveFaceSetNames(Alembic::AbcGeom::IPolyMeshSchema &Schema, TArray<FString>& NamesOut)
{
	// Retrieve face set names to determine if we will have to process face sets (used for face-material indices)
	std::vector<std::string> FaceSetNames;
	Schema.getFaceSetNames(FaceSetNames);

	for (const std::string& Name : FaceSetNames)
	{
		NamesOut.Add(FString(Name.c_str()));
	}

	if (NamesOut.IsEmpty()) 
	{
		NamesOut.Add(FAbcFile::NoFaceSetNameStr);
	}
}

void AbcImporterUtilities::TriangulateIndexBuffer(const TArray<uint32>& InFaceCounts, TArray<uint32>& InOutIndices)
{
	check(InFaceCounts.Num() > 0);
	check(InOutIndices.Num() > 0);

	TArray<uint32> NewIndices;
	NewIndices.Reserve(InFaceCounts.Num() * 4);

	uint32 Index = 0;
	for (const uint32 NumIndicesForFace : InFaceCounts)
	{			
		if (NumIndicesForFace > 3)
		{
			// Triangle 0
			NewIndices.Add(InOutIndices[Index]);
			NewIndices.Add(InOutIndices[Index + 1]);
			NewIndices.Add(InOutIndices[Index + 3]);


			// Triangle 1
			NewIndices.Add(InOutIndices[Index + 3]);
			NewIndices.Add(InOutIndices[Index + 1]);
			NewIndices.Add(InOutIndices[Index + 2]);
		}
		else
		{
			NewIndices.Add(InOutIndices[Index]);
			NewIndices.Add(InOutIndices[Index + 1]);
			NewIndices.Add(InOutIndices[Index + 2]);
		}

		Index += NumIndicesForFace;
	}

	// Set new data
	InOutIndices = NewIndices;
}

void AbcImporterUtilities::TriangulateMaterialIndices(const TArray<uint32>& InFaceCounts, TArray<int32>& InOutData)
{
	check(InFaceCounts.Num() > 0);
	check(InOutData.Num() > 0);

	TArray<int32> NewData;		
	NewData.Reserve(InFaceCounts.Num() * 2);

	for (int32 Index = 0; Index < InFaceCounts.Num(); ++Index)
	{
		const uint32 NumIndicesForFace = InFaceCounts[Index];
		if (NumIndicesForFace == 4)
		{
			NewData.Add(InOutData[Index]);
			NewData.Add(InOutData[Index]);
		}
		else
		{
			NewData.Add(InOutData[Index]);
		}
	}

	// Set new data
	InOutData = NewData;
}

FAbcMeshSample* AbcImporterUtilities::GenerateAbcMeshSampleForFrame(const Alembic::AbcGeom::IPolyMeshSchema& Schema, const Alembic::Abc::ISampleSelector FrameSelector, const ESampleReadFlags ReadFlags, const bool bFirstFrame)
{
	FAbcMeshSample* Sample = new FAbcMeshSample();

	if (!GenerateAbcMeshSampleDataForFrame(Schema, FrameSelector, Sample, ReadFlags, bFirstFrame))
	{
		delete Sample;
		Sample = nullptr;
	}

	return Sample;
}

ESampleReadFlags AbcImporterUtilities::GenerateAbcMeshSampleReadFlags(const Alembic::AbcGeom::IPolyMeshSchema& Schema)
{
	ESampleReadFlags Flags = ESampleReadFlags::Default;

	if (Schema.getPositionsProperty().valid() && !Schema.getPositionsProperty().isConstant())
	{
		Flags |= ESampleReadFlags::Positions;
	}

	if (Schema.getVelocitiesProperty().valid() && !Schema.getVelocitiesProperty().isConstant())
	{
		Flags |= ESampleReadFlags::Velocities;
	}

	if (Schema.getFaceIndicesProperty().valid() && !Schema.getFaceIndicesProperty().isConstant())
	{
		Flags |= ESampleReadFlags::Indices;
	}
	
	if (Schema.getNormalsParam().valid() && !Schema.getNormalsParam().isConstant())
	{
		Flags |= ESampleReadFlags::Normals;
	}

	bool bConstantUVs = Schema.getUVsParam().valid() && Schema.getUVsParam().isConstant();
		Alembic::AbcGeom::ICompoundProperty GeomParams = Schema.getArbGeomParams();
	if (GeomParams.valid() && !bConstantUVs)
	{
		const int32 NumGeomParams = GeomParams.getNumProperties();
		for (int32 GeomParamIndex = 0; GeomParamIndex < NumGeomParams; ++GeomParamIndex)
		{
			Alembic::AbcGeom::IV2fGeomParam UVSetProperty;
			auto PropertyHeader = GeomParams.getPropertyHeader(GeomParamIndex);
			if (Alembic::AbcGeom::IV2fGeomParam::matches(PropertyHeader))
			{
				UVSetProperty = Alembic::AbcGeom::IV2fGeomParam(GeomParams, PropertyHeader.getName());
				bConstantUVs &= UVSetProperty.isConstant();
			}
		}
	}

	if (!bConstantUVs)
	{
		Flags |= ESampleReadFlags::UVs;
	}

	Alembic::AbcGeom::IC3fGeomParam Color3Property;
	Alembic::AbcGeom::IC4fGeomParam Color4Property;

	bool bConstantColors = true;
	if (GeomParams.valid())
	{
		const int32 NumGeomParams = GeomParams.getNumProperties();
		for (int32 GeomParamIndex = 0; GeomParamIndex < NumGeomParams; ++GeomParamIndex)
		{
			auto PropertyHeader = GeomParams.getPropertyHeader(GeomParamIndex);
			if (Alembic::AbcGeom::IC3fGeomParam::matches(PropertyHeader))
			{
				Color3Property = Alembic::AbcGeom::IC3fGeomParam(GeomParams, PropertyHeader.getName());
				bConstantColors &= Color3Property.isConstant();
			}
			else if (Alembic::AbcGeom::IC4fGeomParam::matches(PropertyHeader))
			{
				Color4Property = Alembic::AbcGeom::IC4fGeomParam(GeomParams, PropertyHeader.getName());
				bConstantColors &= Color4Property.isConstant();
			}
		}
	}

	if (!bConstantColors)
	{
		Flags |= ESampleReadFlags::Colors;
	}

	{
		Alembic::AbcGeom::IPolyMeshSchema* MutableSchema = const_cast<Alembic::AbcGeom::IPolyMeshSchema*>(&Schema);
		std::vector<std::string> FaceSetNames;
		MutableSchema->getFaceSetNames(FaceSetNames);
		bool bConstantFaceSets = true;
		for (int32 FaceSetIndex = 0; FaceSetIndex < FaceSetNames.size(); ++FaceSetIndex)
		{
			const std::string& FaceSetName = FaceSetNames[FaceSetIndex];

			if (!MutableSchema->hasFaceSet(FaceSetName))
			{
				continue;
			}

			Alembic::AbcGeom::IFaceSet FaceSet = MutableSchema->getFaceSet(FaceSetName);
			Alembic::AbcGeom::IFaceSetSchema FaceSetSchema = FaceSet.getSchema();
			bConstantFaceSets &= FaceSetSchema.isConstant();
		}

		// Currently face sets are not animated when coming from Maya, so this screws us over :)
		//if (true || !bConstantFaceSets)
		{
			Flags |= ESampleReadFlags::MaterialIndices;
		}
	}
	

	return Flags;
}

/** Generated smoothing groups based on the given face normals, will compare angle between adjacent normals to determine whether or not an edge is hard/soft
	and calculates the smoothing group information with the edge data */
void AbcImporterUtilities::GenerateSmoothingGroups(TMultiMap<uint32, uint32> &TouchingFaces, const TArray<FVector3f>& FaceNormals,
	TArray<uint32>& FaceSmoothingGroups, uint32& HighestSmoothingGroup, const float HardAngleDotThreshold)
{
	// Cache whether or not the hard angle thresshold is set to 0.0 by the user
	const bool bZeroThreshold = FMath::IsNearlyZero(HardAngleDotThreshold);

	// MultiMap holding connected face indices of which is determined they belong to the same smoothing group (angle between face normals tested)
	TMultiMap<uint32, uint32> SmoothingGroupConnectedFaces;
	// Loop over all the faces
	const int32 NumFaces = FaceNormals.Num();
	SmoothingGroupConnectedFaces.Reserve(NumFaces * 3);
	for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
	{
		// Retrieve all the indices to faces that are connect to the current face
		TArray<uint32> ConnectedFaceIndices;
		TouchingFaces.MultiFind(FaceIndex, ConnectedFaceIndices);

		// Get the vertex-averaged face normal
		const FVector3f FaceNormal = FaceNormals[FaceIndex];

		for (int32 i = 0; i < ConnectedFaceIndices.Num(); ++i)
		{
			const uint32 ConnectedFaceIndex = ConnectedFaceIndices[i];
			const FVector3f ConnectedFaceNormal = FaceNormals[ConnectedFaceIndex];

			// Calculate the Angle between the two connected face normals and clamp from 0-1
			const float DotProduct = FMath::Clamp(FMath::Abs(FaceNormal | ConnectedFaceNormal), 0.0f, 1.0f);

			// Compare DotProduct against threshold and handle 0.0 case correctly
			if (DotProduct > HardAngleDotThreshold || (bZeroThreshold && FMath::IsNearlyZero(DotProduct)))
			{
				// If the faces have a "similar" normal we can determine that they should belong to the same smoothing group so mark them as SmoothingGroupConnectedFaces
				SmoothingGroupConnectedFaces.Add(FaceIndex, ConnectedFaceIndex);
				SmoothingGroupConnectedFaces.Add(ConnectedFaceIndex, FaceIndex);
			}
		}

		FaceSmoothingGroups[FaceIndex] = INDEX_NONE;
	}

	
	TArray<TArray<uint32, TInlineAllocator<12>>> FaceData;
	int32 FaceIndex = 0;
	int32 CurrentFaceIndex = 0;
	int32 CurrentRecursionDepth = 0;
	int32 PreviousRecursionDepth = 0;
	int32 ProcessedFaces = 1;
	int32 SmoothingGroupIndex = 0;

	// While number of processed face is 
	while (ProcessedFaces != NumFaces && CurrentFaceIndex < NumFaces)
	{
		// Check if there is valid scratch data available
		if (!FaceData.IsValidIndex(CurrentRecursionDepth))
		{
			FaceData.AddDefaulted((CurrentRecursionDepth + 1) - FaceData.Num());
		}
		
		// Retrieve scratch data for this recursion depth
		TArray<uint32, TInlineAllocator<12>>& ConnectedFaceIndices = FaceData[CurrentRecursionDepth];

		// Retrieve connected faces if we moved down a step
		if (PreviousRecursionDepth <= CurrentRecursionDepth)
		{
				ConnectedFaceIndices.Empty();

				// Check if this face has already been processed (assigned a face index)
				if (FaceSmoothingGroups[CurrentFaceIndex] == INDEX_NONE)
				{
					SmoothingGroupConnectedFaces.MultiFind(CurrentFaceIndex, ConnectedFaceIndices);
					FaceSmoothingGroups[CurrentFaceIndex] = SmoothingGroupIndex;
				}
				else
				{
					// If so step up to top recursion level and increment face index to process next
					CurrentFaceIndex = ++FaceIndex;
					CurrentRecursionDepth = 0;
					continue;
				}
			}

			// Store recursion depth for next cycle
			PreviousRecursionDepth = CurrentRecursionDepth;

			// If there are any connected face check if they still need to be processed
			if (ConnectedFaceIndices.Num())
			{
				int32 FoundFaceIndex = INDEX_NONE;
				for (int32 FoundConnectedFaceIndex = 0; FoundConnectedFaceIndex < ConnectedFaceIndices.Num(); ++FoundConnectedFaceIndex)
			{
					const int32 ConnectedFaceIndex = ConnectedFaceIndices[FoundConnectedFaceIndex];
					if (FaceSmoothingGroups[ConnectedFaceIndex] == INDEX_NONE)
					{
						FoundFaceIndex = ConnectedFaceIndex;

						// Step down for next cycle
						++CurrentRecursionDepth;
						++ProcessedFaces;
						break;
					}
			}

				if (FoundFaceIndex != INDEX_NONE)
				{
					// Set next face index to process
					CurrentFaceIndex = FoundFaceIndex;
					// Remove the index from the connected faces list as it'll be processed
					ConnectedFaceIndices.Remove(CurrentFaceIndex);
				}
				else
				{
					// No connected faces left so step up
					--CurrentRecursionDepth;
				}
			}
			else
			{
				// No connected faces left so step up
				--CurrentRecursionDepth;
			}

			// If we reached the top of recursion stack reset the values
			if (CurrentRecursionDepth == -1)
			{
				CurrentFaceIndex = ++FaceIndex;
				CurrentRecursionDepth = 0;
				++SmoothingGroupIndex;
		}
	}	

	HighestSmoothingGroup = SmoothingGroupIndex;
}

bool AbcImporterUtilities::GenerateAbcMeshSampleDataForFrame(const Alembic::AbcGeom::IPolyMeshSchema &Schema, const Alembic::Abc::ISampleSelector FrameSelector, FAbcMeshSample* &Sample, const ESampleReadFlags ReadFlags, const bool bFirstFrame)
{
	// Get single (vertex-data) sample from Alembic file
	Alembic::AbcGeom::IPolyMeshSchema::Sample MeshSample;
	Schema.get(MeshSample, FrameSelector);

	bool bRetrievalResult = true;

	// Retrieve all available mesh data
	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Positions))
	{
		Alembic::Abc::P3fArraySamplePtr PositionsSample = MeshSample.getPositions();
		bRetrievalResult &= RetrieveTypedAbcData<Alembic::Abc::P3fArraySamplePtr, FVector3f>(PositionsSample, Sample->Vertices);
		if (!bRetrievalResult)
		{
			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("NoVertices", "Could not retrieve mesh vertices."));
			FAbcImportLogger::AddImportMessage(Message);
			return false;
		}
	}
	
	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Velocities))
	{
		const bool bVelocitiesAvailable = Schema.getVelocitiesProperty().valid();

		if (bVelocitiesAvailable)
		{
			Alembic::Abc::V3fArraySamplePtr VelocitiesSample = MeshSample.getVelocities();
			bRetrievalResult &= RetrieveTypedAbcData<Alembic::Abc::V3fArraySamplePtr, FVector3f>(VelocitiesSample, Sample->Velocities);
			if (!bRetrievalResult)
			{
				TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("NoVelocities", "Could not retrieve velocities. Try importing without velocities as motion vectors."));
				FAbcImportLogger::AddImportMessage(Message);
				return false;
			}
		}
	}

	TArray<uint32> FaceCounts;	
	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Indices | ESampleReadFlags::UVs | ESampleReadFlags::Normals | ESampleReadFlags::Colors | ESampleReadFlags::MaterialIndices))
	{
		Alembic::Abc::Int32ArraySamplePtr FaceCountsSample = MeshSample.getFaceCounts();
		bRetrievalResult &= RetrieveTypedAbcData<Alembic::Abc::Int32ArraySamplePtr, uint32>(FaceCountsSample, FaceCounts);
		if (!bRetrievalResult)
		{
			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("NoFaceCounts", "Could not retrieve mesh face counts."));
			FAbcImportLogger::AddImportMessage(Message);
			return false;
		}
	}

	const bool bNeedsTriangulation = FaceCounts.Contains(4);
	if (bFirstFrame)
	{
		const uint32* Result = FaceCounts.FindByPredicate([](uint32 FaceCount) { return FaceCount < 3 || FaceCount > 4; });
		if (Result)
		{
			// We found an Ngon which we can't triangulate atm
			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("FoundNGon", "Unable to import mesh due to a face consisting of {0} vertices, expecting triangles (3) or quads (4)."), FText::FromString(FString::FromInt((*Result)))));
			FAbcImportLogger::AddImportMessage(Message);
			return false;
		}
	}

	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Indices))
	{
		Alembic::Abc::Int32ArraySamplePtr IndicesSample = MeshSample.getFaceIndices();
		bRetrievalResult &= RetrieveTypedAbcData<Alembic::Abc::Int32ArraySamplePtr, uint32>(IndicesSample, Sample->Indices);
		if (bRetrievalResult)
		{
			// Validate that the indices coming from the Alembic are valid, in case it is corrupted
			const uint32 MaxIndex = IntCastChecked<uint32>(Sample->Vertices.Num());
			for (uint32& Index : Sample->Indices)
			{
				if (Index >= MaxIndex)
				{
					TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FoundInvalidIndex", "Unable to import mesh due to index out of bound found."));
					FAbcImportLogger::AddImportMessage(Message);
					return false;
				}
			}
		}
		else
		{
			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("NoIndices", "Could not retrieve mesh indices."));
			FAbcImportLogger::AddImportMessage(Message);
			return false;
		}

		if (bNeedsTriangulation)
		{
			TriangulateIndexBuffer(FaceCounts, Sample->Indices);
		}
	}

	Alembic::AbcGeom::ICompoundProperty GeomParams = Schema.getArbGeomParams();
	
	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::UVs))
	{
		Alembic::AbcGeom::IV2fGeomParam UVCoordinateParameter = Schema.getUVsParam();
		if (UVCoordinateParameter.valid())
		{
			ReadUVSetData(UVCoordinateParameter, FrameSelector, Sample->UVs[0], Sample->Indices, bNeedsTriangulation, FaceCounts, Sample->Vertices.Num());
		}
		else
		{
			Sample->UVs[0].AddZeroed(Sample->Indices.Num());
		}

		if (GeomParams.valid())
		{
			const int32 NumGeomParams = GeomParams.getNumProperties();
			for (int32 GeomParamIndex = 0; GeomParamIndex < NumGeomParams; ++GeomParamIndex)
			{
				Alembic::AbcGeom::IV2fGeomParam UVSetProperty;
				auto PropertyHeader = GeomParams.getPropertyHeader(GeomParamIndex);
				if (Alembic::AbcGeom::IV2fGeomParam::matches(PropertyHeader))
				{
					if (Sample->NumUVSets < MAX_TEXCOORDS)
					{
						UVSetProperty = Alembic::AbcGeom::IV2fGeomParam(GeomParams, PropertyHeader.getName());
						ReadUVSetData(UVSetProperty, FrameSelector, Sample->UVs[Sample->NumUVSets], Sample->Indices, bNeedsTriangulation, FaceCounts, Sample->Vertices.Num());
						++Sample->NumUVSets;
					}
					else
					{
						TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("MaxUVs", "More than 4 UV sets found. The remaining UV sets will be ignored."));
						FAbcImportLogger::AddImportMessage(Message);
						break;
					}
				}
			}
		}
	}

	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Normals))
	{
		Alembic::AbcGeom::IN3fGeomParam NormalParameter = Schema.getNormalsParam();
		// Check if Normals are available anyhow
		const bool bNormalsAvailable = NormalParameter.valid();

		// Check if the Normals are 'constant' which means there won't be any normal data available after frame 0
		bool bConstantNormals = bNormalsAvailable && Schema.getNormalsParam().isConstant();
		if (bNormalsAvailable && (!bConstantNormals || (bConstantNormals && bFirstFrame)))
		{
			Alembic::Abc::N3fArraySamplePtr NormalsSample = NormalParameter.getValueProperty().getValue(FrameSelector);
			RetrieveTypedAbcData<Alembic::Abc::N3fArraySamplePtr, FVector3f>(NormalsSample, Sample->Normals);

			if (Sample->Normals.Num() > 0)
			{
				// Can only retrieve normal indices when the Normals array is indexed
				bool bIndexedNormals = NormalParameter.getIndexProperty().valid();
				if (bIndexedNormals)
				{
					Alembic::Abc::UInt32ArraySamplePtr NormalIndiceSample = NormalParameter.getIndexProperty().getValue(FrameSelector);
					TArray<uint32> NormalIndices;
					RetrieveTypedAbcData<Alembic::Abc::UInt32ArraySamplePtr, uint32>(NormalIndiceSample, NormalIndices);

					if (bNeedsTriangulation)
					{
						TriangulateIndexBuffer(FaceCounts, NormalIndices);
					}

					// Expand Normal array
					ExpandVertexAttributeArray<FVector3f>(NormalIndices, Sample->Normals);
				}
				else
				{
					ProcessVertexAttributeArray(Sample->Indices, FaceCounts, bNeedsTriangulation, Sample->Vertices.Num(), Sample->Normals);
				}

				// Make sure the normals from the Alembic are really normalized
				ParallelFor(Sample->Normals.Num(), [&Sample](int32 Index)
				{
					Sample->Normals[Index].Normalize();
				});
			}
			else if (bFirstFrame)
			{
				TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("NoNormals", "Normals were supposed to be available but none were found."));
				FAbcImportLogger::AddImportMessage(Message);
			}
		}
	}

	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Colors))
	{
		Alembic::AbcGeom::IC3fGeomParam Color3Property;
		Alembic::AbcGeom::IC4fGeomParam Color4Property;
		if (GeomParams.valid())
		{
			const int32 NumGeomParams = GeomParams.getNumProperties();
			for (int32 GeomParamIndex = 0; GeomParamIndex < NumGeomParams; ++GeomParamIndex)
			{
				auto PropertyHeader = GeomParams.getPropertyHeader(GeomParamIndex);
				if (Alembic::AbcGeom::IC3fGeomParam::matches(PropertyHeader))
				{
					Color3Property = Alembic::AbcGeom::IC3fGeomParam(GeomParams, PropertyHeader.getName());
				}
				else if (Alembic::AbcGeom::IC4fGeomParam::matches(PropertyHeader))
				{
					Color4Property = Alembic::AbcGeom::IC4fGeomParam(GeomParams, PropertyHeader.getName());
				}
			}
		}

		if (Color3Property.valid())
		{
			Alembic::Abc::C3fArraySamplePtr ColorSample = Color3Property.getValueProperty().getValue(FrameSelector);

			// Allocate required memory for the OutData
			const int32 NumEntries = ColorSample->size();

			bool bSuccess = false;

			if (NumEntries)
			{
				Sample->Colors.AddZeroed(NumEntries);

				for (int32 Entry = 0; Entry < NumEntries; ++Entry)
				{
					auto DataPtr = &ColorSample->get()[Entry];
					auto OutDataPtr = &Sample->Colors[Entry];
					FMemory::Memcpy(OutDataPtr, DataPtr, sizeof(FLinearColor));
					Sample->Colors[Entry].A = 1.0f;
				}
			}

			const bool bIndexedColors = Color3Property.getIndexProperty().valid();
			if (bIndexedColors)
			{
				Alembic::Abc::UInt32ArraySamplePtr ColorIndiceSample = Color3Property.getIndexProperty().getValue(FrameSelector);
				TArray<uint32> ColorIndices;
				RetrieveTypedAbcData<Alembic::Abc::UInt32ArraySamplePtr, uint32>(ColorIndiceSample, ColorIndices);

				if (bNeedsTriangulation)
				{
					TriangulateIndexBuffer(FaceCounts, ColorIndices);
				}

				// Expand color array
				ExpandVertexAttributeArray<FLinearColor>(ColorIndices.Num() ? ColorIndices : Sample->Indices, Sample->Colors);
			}
			else
			{
				ProcessVertexAttributeArray(Sample->Indices, FaceCounts, bNeedsTriangulation, Sample->Vertices.Num(), Sample->Colors);
			}
		}
		else if (Color4Property.valid())
		{
			Alembic::AbcGeom::IC4fGeomParam::Sample sample;
			Color4Property.getExpanded(sample, FrameSelector);
			Alembic::Abc::C4fArraySamplePtr ColorSample = Color4Property.getValueProperty().getValue(FrameSelector);
			RetrieveTypedAbcData<Alembic::Abc::C4fArraySamplePtr, FLinearColor>(ColorSample, Sample->Colors);

			bool bIndexedColors = Color4Property.getIndexProperty().valid();
			if (bIndexedColors)
			{
				Alembic::Abc::UInt32ArraySamplePtr ColorIndiceSample = Color4Property.getIndexProperty().getValue(FrameSelector);
				TArray<uint32> Indices;
				RetrieveTypedAbcData<Alembic::Abc::UInt32ArraySamplePtr, uint32>(ColorIndiceSample, Indices);

				if (bNeedsTriangulation)
				{
					TriangulateIndexBuffer(FaceCounts, Indices);
				}

				// Expand color array
				ExpandVertexAttributeArray<FLinearColor>(Indices.Num() ? Indices : Sample->Indices, Sample->Colors);
			}
			else
			{
				ProcessVertexAttributeArray(Sample->Indices, FaceCounts, bNeedsTriangulation, Sample->Vertices.Num(), Sample->Colors);
			}
		}
		else
		{
			Sample->Colors.AddZeroed(Sample->Indices.Num());
		}
	}
	else
	{
		if (Sample->Colors.Num() < Sample->Indices.Num())
		{
			Sample->Colors.AddZeroed(Sample->Indices.Num() - Sample->Colors.Num());
		}
	}

	// Pre initialize face-material indices
	Sample->MaterialIndices.AddZeroed(Sample->Indices.Num() / 3);
	Sample->NumMaterials = GenerateMaterialIndicesFromFaceSets(*const_cast<Alembic::AbcGeom::IPolyMeshSchema*>(&Schema), FrameSelector, Sample->MaterialIndices);

	// Triangulate material face indices if needed
	if (bNeedsTriangulation)
	{
		TriangulateMaterialIndices(FaceCounts, Sample->MaterialIndices);
	}

	return bRetrievalResult;
}

void AbcImporterUtilities::ReadUVSetData(Alembic::AbcGeom::IV2fGeomParam &UVCoordinateParameter, const Alembic::Abc::ISampleSelector FrameSelector, TArray<FVector2f>& OutUVs, const TArray<uint32>& MeshIndices, const bool bNeedsTriangulation, const TArray<uint32>& FaceCounts, const int32 NumVertices)
{
	Alembic::Abc::V2fArraySamplePtr UVSample = UVCoordinateParameter.getValueProperty().getValue(FrameSelector);
	RetrieveTypedAbcData<Alembic::Abc::V2fArraySamplePtr, FVector2f>(UVSample, OutUVs);

	// Can only retrieve UV indices when the UVs array is indexed
	const bool bIndexedUVs = UVCoordinateParameter.getIndexProperty().valid();
	if (bIndexedUVs)
	{
		Alembic::Abc::UInt32ArraySamplePtr UVIndiceSample = UVCoordinateParameter.getIndexProperty().getValue(FrameSelector);
		TArray<uint32> UVIndices;
		const bool bSuccess = RetrieveTypedAbcData<Alembic::Abc::UInt32ArraySamplePtr, uint32>(UVIndiceSample, UVIndices);
		if (bSuccess)
		{
			if (bNeedsTriangulation)
			{
				TriangulateIndexBuffer(FaceCounts, UVIndices);
			}

			// Expand UV array
			ExpandVertexAttributeArray<FVector2f>(UVIndices, OutUVs);
		}
		else
		{
			// Fallback to ensure there's always UV data present since it's needed later to compute tangents
			OutUVs.AddZeroed(MeshIndices.Num());
		}
	}
	else if (OutUVs.Num())
	{
		ProcessVertexAttributeArray(MeshIndices, FaceCounts, bNeedsTriangulation, NumVertices, OutUVs);
	}
}

void AbcImporterUtilities::GenerateSmoothingGroupsIndices(FAbcMeshSample* MeshSample, float HardEdgeAngleThreshold)
{
	// Vertex lookup map
	TMultiMap<uint32, uint32> VertexLookupMap;

	// Stores face indices that touch (at either one of their vertices)
	TMultiMap<uint32, uint32> TouchingFaces;

	// Stores the individual face normals (vertex averaged)
	TArray<FVector3f> FaceNormals;

	// Pre-initialize RawMesh arrays
	const int32 NumFaces = MeshSample->Indices.Num() / 3;
	MeshSample->SmoothingGroupIndices.Empty(NumFaces);
	MeshSample->SmoothingGroupIndices.AddZeroed(NumFaces);

	// Loop over faces
	uint32 Offset = 0;

	for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
	{
		// Will hold the averaged face normal
		FVector3f FaceNormal(0, 0, 0);

		// Determine number of vertices for this face (we only support triangle-based meshes for now)
		const int32 NumVertsForFace = 3;
			
		// Triangle index winding
		const int32 TriangleIndices[3] = { 2, 1, 0 };

		// Loop over verts for current face (only support triangulated)
		for (int32 CornerIndex = 0; CornerIndex < NumVertsForFace; ++CornerIndex)
		{
			// Sample and face corner offset
			const uint32 TriSampleIndex = Offset + TriangleIndices[CornerIndex];
			const uint32 CornerOffset = Offset + CornerIndex;

			// Vertex, uv and normal indices
			const int32 VertexIndex = MeshSample->Indices[TriSampleIndex];

			// Check if there is already information stored for this VertexIndex
			TArray<const uint32*> VertexInformations;
			VertexLookupMap.MultiFindPointer(VertexIndex, VertexInformations);

			// If it doesn't add a new entry with storing the current FaceIndex
			if (VertexInformations.Num() == 0)
			{
				VertexLookupMap.Add(VertexIndex, FaceIndex);
			}
			else
			{
				// If there is an entry found (can be multiple)
				bool bFound = false;
				for (int32 VertexInfoIndex = 0; VertexInfoIndex < VertexInformations.Num(); ++VertexInfoIndex)
				{
					// Check if they belong to the face index, if so we don't have to add another entry
					const uint32* StoredFaceIndex = VertexInformations[VertexInfoIndex];
					if (*StoredFaceIndex == FaceIndex)
					{
						bFound = true;
					}
					else
					{
						// If the VertexIndices are the same but the FaceIndex differs we found two faces that share at least one vertex, thus add them to the TouchFaces map
						TouchingFaces.AddUnique(*StoredFaceIndex, FaceIndex);
					}
				}

				// If we didn't find an entry with the same FaceIndex add a new entry for it
				if (!bFound)
				{
					VertexLookupMap.Add(VertexIndex, FaceIndex);
				}
			}

			// Retrieve normal to calculate the face normal
			FVector3f Normal = MeshSample->Normals[TriSampleIndex];

			// Averaged face normal addition
			FaceNormal += Normal;
		}

		// Moving along the vertex reading position by the amount of vertices for this face
		Offset += NumVertsForFace;

		// Store the averaged face normal
		FaceNormal.Normalize();
		FaceNormals.Add(FaceNormal);
	}
		
	MeshSample->NumSmoothingGroups = 0;
	GenerateSmoothingGroups(TouchingFaces, FaceNormals, MeshSample->SmoothingGroupIndices, MeshSample->NumSmoothingGroups, HardEdgeAngleThreshold);
	MeshSample->NumSmoothingGroups += 1;
}

void AbcImporterUtilities::CalculateNormals(FAbcMeshSample* Sample)
{
	Sample->Normals.Empty(Sample->Indices.Num());
	Sample->Normals.AddZeroed(Sample->Indices.Num());

	const uint32 NumFaces = Sample->Indices.Num() / 3;
	for (uint32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
	{
		// Triangle index winding
		const int32 TriangleIndices[3] = { 2, 1, 0 };
		const int32 FaceOffset = FaceIndex * 3;

		FVector3f VertexPositions[3];
		int32 VertexIndices[3];

		// Retrieve vertex indices and positions
		VertexIndices[0] = Sample->Indices[FaceOffset + TriangleIndices[0]];
		VertexPositions[0] = Sample->Vertices[VertexIndices[0]];

		VertexIndices[1] = Sample->Indices[FaceOffset + TriangleIndices[1]];
		VertexPositions[1] = Sample->Vertices[VertexIndices[1]];

		VertexIndices[2] = Sample->Indices[FaceOffset + TriangleIndices[2]];
		VertexPositions[2] = Sample->Vertices[VertexIndices[2]];


		// Calculate normal for triangle face			
		FVector3f N = FVector3f::CrossProduct((VertexPositions[0] - VertexPositions[1]), (VertexPositions[0] - VertexPositions[2]));
		N.Normalize();

		// Unrolled loop
		Sample->Normals[FaceOffset + 0] += N;
		Sample->Normals[FaceOffset + 1] += N;
		Sample->Normals[FaceOffset + 2] += N;
	}

	for (FVector3f& Normal : Sample->Normals)
	{
		Normal.Normalize();
	}
}

void AbcImporterUtilities::CalculateSmoothNormals(FAbcMeshSample* Sample)
{
	TArray<FVector3f> PerVertexNormals;		
	PerVertexNormals.AddZeroed(Sample->Vertices.Num());

	// Loop over each face
	const uint32 NumFaces = Sample->Indices.Num() / 3;
	for (uint32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
	{
		const int32 TriangleIndices[3] = { 2, 1, 0 };
		const int32 FaceOffset = FaceIndex * 3;
			
		// Retrieve vertex indices and positions
		int32 VertexIndices[3];
		FVector3f VertexPositions[3];
		
		// Retrieve vertex indices and positions
		VertexIndices[0] = Sample->Indices[FaceOffset + TriangleIndices[0]];
		VertexPositions[0] = Sample->Vertices[VertexIndices[0]];

		VertexIndices[1] = Sample->Indices[FaceOffset + TriangleIndices[1]];
		VertexPositions[1] = Sample->Vertices[VertexIndices[1]];

		VertexIndices[2] = Sample->Indices[FaceOffset + TriangleIndices[2]];
		VertexPositions[2] = Sample->Vertices[VertexIndices[2]];
			
		// Calculate normal for triangle face			
		FVector3f N = FVector3f::CrossProduct((VertexPositions[0] - VertexPositions[1]), (VertexPositions[0] - VertexPositions[2]));
		N.Normalize();

		// Unrolled loop
		PerVertexNormals[VertexIndices[0]] += N;
		PerVertexNormals[VertexIndices[1]] += N;
		PerVertexNormals[VertexIndices[2]] += N;
	}

	Sample->Normals.Empty(Sample->Indices.Num());
	Sample->Normals.AddZeroed(Sample->Indices.Num());

	for (uint32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
	{
		const int32 FaceOffset = FaceIndex * 3;

		// Unrolled loop for calculating final normals
		Sample->Normals[FaceOffset + 0] = PerVertexNormals[Sample->Indices[FaceOffset + 0]];
		Sample->Normals[FaceOffset + 0].Normalize();

		Sample->Normals[FaceOffset + 1] = PerVertexNormals[Sample->Indices[FaceOffset + 1]];
		Sample->Normals[FaceOffset + 1].Normalize();

		Sample->Normals[FaceOffset + 2] = PerVertexNormals[Sample->Indices[FaceOffset + 2]];
		Sample->Normals[FaceOffset + 2].Normalize();
	}
}

void AbcImporterUtilities::CalculateNormalsWithSmoothingGroups(FAbcMeshSample* Sample, const TArray<uint32>& SmoothingMasks, const uint32 NumSmoothingGroups)
{
	if (NumSmoothingGroups == 1)
	{
		CalculateSmoothNormals(Sample);
		return;
	}

	TArray<FVector3f> PerVertexNormals;
	PerVertexNormals.AddZeroed(Sample->Vertices.Num());	

	TMap<TPair<uint32, uint32>, FVector3f> SmoothingGroupVertexNormals;
	SmoothingGroupVertexNormals.Reserve(Sample->Indices.Num());
	
	// Loop over each face
	const uint32 NumFaces = Sample->Indices.Num() / 3;
	const int32 TriangleIndices[3] = { 2, 1, 0 };
	int32 VertexIndices[3];
	FVector3f VertexPositions[3];

	for (uint32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
	{
		// Retrieve smoothing group for this face
		const int32 SmoothingGroup = SmoothingMasks[FaceIndex];
		const int32 FaceOffset = FaceIndex * 3;

		// Retrieve vertex indices and positions
		VertexIndices[0] = Sample->Indices[FaceOffset + TriangleIndices[0]];
		VertexPositions[0] = Sample->Vertices[VertexIndices[0]];

		VertexIndices[1] = Sample->Indices[FaceOffset + TriangleIndices[1]];
		VertexPositions[1] = Sample->Vertices[VertexIndices[1]];

		VertexIndices[2] = Sample->Indices[FaceOffset + TriangleIndices[2]];
		VertexPositions[2] = Sample->Vertices[VertexIndices[2]];

		// Calculate normal for triangle face			
		FVector3f N = FVector3f::CrossProduct((VertexPositions[0] - VertexPositions[1]), (VertexPositions[0] - VertexPositions[2]));
		N.Normalize();				

		for (int32 Index = 0; Index < 3; ++Index)
		{
			const TPair<uint32, uint32> Pair = TPair<uint32, uint32>(SmoothingGroup, VertexIndices[Index]);
			if (FVector3f* SN = SmoothingGroupVertexNormals.Find(Pair))
			{
				(*SN) += N;
			}
			else
			{
				SmoothingGroupVertexNormals.Add(Pair, N);
			}
		}		
	}

	Sample->Normals.Empty(Sample->Indices.Num());
	Sample->Normals.AddZeroed(Sample->Indices.Num());

	for (uint32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
	{
		// Retrieve smoothing group for this face
		const int32 SmoothingGroup = SmoothingMasks[FaceIndex];
		const int32 FaceOffset = FaceIndex * 3;			

		for (int32 Index = 0; Index < 3; ++Index)
		{
			Sample->Normals[FaceOffset + Index] = SmoothingGroupVertexNormals.FindChecked(TPair<uint32, uint32>(SmoothingGroup, Sample->Indices[FaceOffset + Index]));
			Sample->Normals[FaceOffset + Index].Normalize();
		}
	}
}

void AbcImporterUtilities::CalculateNormalsWithSampleData(FAbcMeshSample* Sample, const FAbcMeshSample* SourceSample)
{
	CalculateNormalsWithSmoothingGroups(Sample, SourceSample->SmoothingGroupIndices, SourceSample->NumSmoothingGroups);
	Sample->SmoothingGroupIndices = SourceSample->SmoothingGroupIndices;
	Sample->NumSmoothingGroups = SourceSample->NumSmoothingGroups;
}

void AbcImporterUtilities::ComputeTangents(FAbcMeshSample* Sample, bool bIgnoreDegenerateTriangles, IMeshUtilities& MeshUtilities)
{
	// At this point, the normals are already computed/retrieved from the Alembic
	// so directly call CalculateMikkTSpaceTangents, which won't compute normals
	MeshUtilities.CalculateMikkTSpaceTangents(Sample->Vertices, Sample->Indices, Sample->UVs[0], Sample->Normals, bIgnoreDegenerateTriangles, Sample->TangentX, Sample->TangentY);
}

FAbcMeshSample* AbcImporterUtilities::MergeMeshSamples(const TArray<const FAbcMeshSample*>& Samples)
{
	FAbcMeshSample* MergedSample = new FAbcMeshSample();
	FMemory::Memzero(MergedSample, sizeof(FAbcMeshSample));

	for (const FAbcMeshSample* Sample : Samples)
	{
		const uint32 VertexOffset = MergedSample->Vertices.Num();
		MergedSample->Vertices.Append(Sample->Vertices);
			
		const uint32 IndicesOffset = MergedSample->Indices.Num();
		MergedSample->Indices.Append(Sample->Indices);
			
		// Remap indices
		const uint32 NumIndices = MergedSample->Indices.Num();
		for (uint32 IndiceIndex = IndicesOffset; IndiceIndex < NumIndices; ++IndiceIndex)
		{
			MergedSample->Indices[IndiceIndex] += VertexOffset;
		}

		// Vertex attributes (per index based)
		MergedSample->Normals.Append(Sample->Normals);
		MergedSample->TangentX.Append(Sample->TangentX);
		MergedSample->TangentY.Append(Sample->TangentY);

		// Add valid number of UVs and zero padding for unavailable UV channels
		MergedSample->UVs[0].Append(Sample->UVs[0]);
		if (Sample->NumUVSets >= MergedSample->NumUVSets)
		{
			for (uint32 UVIndex = 1; UVIndex < Sample->NumUVSets; ++UVIndex)
			{
				const int32 NumMissingUVs = (MergedSample->UVs[0].Num() - MergedSample->UVs[UVIndex].Num()) - Sample->UVs[UVIndex].Num();
				MergedSample->UVs[UVIndex].AddZeroed(NumMissingUVs);
				MergedSample->UVs[UVIndex].Append(Sample->UVs[UVIndex]);
			}

			MergedSample->NumUVSets = Sample->NumUVSets;
		}
		else if (Sample->NumUVSets < MergedSample->NumUVSets)
		{
			for (uint32 UVIndex = 1; UVIndex < MergedSample->NumUVSets; ++UVIndex)
			{
				MergedSample->UVs[UVIndex].AddZeroed(Sample->UVs[0].Num());
			}
		}		

		// Currently not used but will still merge
		MergedSample->Colors.Append(Sample->Colors);
		/*MergedSample->Visibility.Append(Sample->Visibility);
		MergedSample->VisibilityIndices.Append(Sample->VisibilityIndices);*/

		const uint32 MaterialIndicesOffset = MergedSample->MaterialIndices.Num();
		const uint32 SmoothingGroupIndicesOffset = MergedSample->SmoothingGroupIndices.Num();

		ensureMsgf(MaterialIndicesOffset == SmoothingGroupIndicesOffset, TEXT("Material and smoothing group indices count should match (%d != %d))"), MaterialIndicesOffset, SmoothingGroupIndicesOffset);

		// Per Face material and smoothing group index
		MergedSample->MaterialIndices.Append(Sample->MaterialIndices);
		MergedSample->SmoothingGroupIndices.Append(Sample->SmoothingGroupIndices);

		// Remap material and smoothing group indices in separate loops since they might not have the same number of elements
		const uint32 NumMaterialIndices = MergedSample->MaterialIndices.Num();
		for (uint32 IndiceIndex = MaterialIndicesOffset; IndiceIndex < NumMaterialIndices; ++IndiceIndex)
		{
			MergedSample->MaterialIndices[IndiceIndex] += MergedSample->NumMaterials;
		}

		const uint32 NumSmoothingGroupIndices = MergedSample->SmoothingGroupIndices.Num();
		for (uint32 IndiceIndex = SmoothingGroupIndicesOffset; IndiceIndex < NumSmoothingGroupIndices; ++IndiceIndex)
		{
			MergedSample->SmoothingGroupIndices[IndiceIndex] += MergedSample->NumSmoothingGroups;
		}

		MergedSample->NumSmoothingGroups += (Sample->NumSmoothingGroups != 0 ) ? Sample->NumSmoothingGroups : 1;
		MergedSample->NumMaterials += (Sample->NumMaterials != 0) ? Sample->NumMaterials : 1;			
	}
		
	return MergedSample;
}

FAbcMeshSample* AbcImporterUtilities::MergeMeshSamples(FAbcMeshSample* MeshSampleOne, FAbcMeshSample* MeshSampleTwo)
{
	TArray<const FAbcMeshSample*> Samples;
	Samples.Add(MeshSampleOne);
	Samples.Add(MeshSampleTwo);		
	return MergeMeshSamples(Samples);
}

void AbcImporterUtilities::AppendMeshSample(FAbcMeshSample* MeshSampleOne, const FAbcMeshSample* MeshSampleTwo)
{
	const uint32 VertexOffset = MeshSampleOne->Vertices.Num();
	MeshSampleOne->Vertices.Append(MeshSampleTwo->Vertices);

	MeshSampleOne->Velocities.Append(MeshSampleTwo->Velocities);

	const uint32 IndicesOffset = MeshSampleOne->Indices.Num();
	MeshSampleOne->Indices.Append(MeshSampleTwo->Indices);

	// Remap indices
	const uint32 NumIndices = MeshSampleOne->Indices.Num();
	for (uint32 IndiceIndex = IndicesOffset; IndiceIndex < NumIndices; ++IndiceIndex)
	{
		MeshSampleOne->Indices[IndiceIndex] += VertexOffset;
	}

	// Vertex attributes (per index based)
	MeshSampleOne->Normals.Append(MeshSampleTwo->Normals);
	MeshSampleOne->TangentX.Append(MeshSampleTwo->TangentX);
	MeshSampleOne->TangentY.Append(MeshSampleTwo->TangentY);

	// Append valid number of UVs and zero padding for unavailable UV channels	
	if (MeshSampleTwo->NumUVSets >= MeshSampleOne->NumUVSets)
	{
		for (uint32 UVIndex = 1; UVIndex < MeshSampleTwo->NumUVSets; ++UVIndex)
		{
			const int32 NumMissingUVs = MeshSampleOne->UVs[0].Num() - MeshSampleOne->UVs[UVIndex].Num();
			MeshSampleOne->UVs[UVIndex].AddZeroed(NumMissingUVs);
			MeshSampleOne->UVs[UVIndex].Append(MeshSampleTwo->UVs[UVIndex]);
		}

		MeshSampleOne->NumUVSets = MeshSampleTwo->NumUVSets;
	}
	else
	{
		for (uint32 UVIndex = 1; UVIndex < MeshSampleOne->NumUVSets; ++UVIndex)
		{
			MeshSampleOne->UVs[UVIndex].AddZeroed(MeshSampleTwo->UVs[0].Num());
		}
	}

	MeshSampleOne->UVs[0].Append(MeshSampleTwo->UVs[0]);

	MeshSampleOne->Colors.Append(MeshSampleTwo->Colors);
	// Currently not used but will still merge	
	/*MeshSampleOne->Visibility.Append(MeshSampleTwo->Visibility);
	MeshSampleOne->VisibilityIndices.Append(MeshSampleTwo->VisibilityIndices);*/

	const uint32 MaterialIndicesOffset = MeshSampleOne->MaterialIndices.Num();
	const uint32 SmoothingGroupIndicesOffset = MeshSampleOne->SmoothingGroupIndices.Num();

	ensureMsgf(MaterialIndicesOffset == SmoothingGroupIndicesOffset, TEXT("Material and smoothing group indices count should match (%d != %d))"), MaterialIndicesOffset, SmoothingGroupIndicesOffset);

	// Per Face material and smoothing group index
	MeshSampleOne->MaterialIndices.Append(MeshSampleTwo->MaterialIndices);
	MeshSampleOne->SmoothingGroupIndices.Append(MeshSampleTwo->SmoothingGroupIndices);

	// Remap material and smoothing group indices in separate loops since they might not have the same number of elements
	const uint32 NumMaterialIndices = MeshSampleOne->MaterialIndices.Num();
	for (uint32 IndiceIndex = MaterialIndicesOffset; IndiceIndex < NumMaterialIndices; ++IndiceIndex)
	{
		MeshSampleOne->MaterialIndices[IndiceIndex] += MeshSampleOne->NumMaterials;
	}

	const uint32 NumSmoothingGroupIndices = MeshSampleOne->SmoothingGroupIndices.Num();
	for (uint32 IndiceIndex = SmoothingGroupIndicesOffset; IndiceIndex < NumSmoothingGroupIndices; ++IndiceIndex)
	{
		MeshSampleOne->SmoothingGroupIndices[IndiceIndex] += MeshSampleOne->NumSmoothingGroups;
	}

	MeshSampleOne->NumSmoothingGroups += (MeshSampleTwo->NumSmoothingGroups != 0) ? MeshSampleTwo->NumSmoothingGroups : 1;
	MeshSampleOne->NumMaterials += (MeshSampleTwo->NumMaterials != 0) ? MeshSampleTwo->NumMaterials : 1;
}

void AbcImporterUtilities::GetHierarchyForObject(const Alembic::Abc::IObject& Object, TDoubleLinkedList<Alembic::AbcGeom::IXform>& Hierarchy)
{
	Alembic::Abc::IObject Parent;
	Parent = Object.getParent();

	// Traverse through parents until we reach RootNode
	while (Parent.valid())
	{
		// Only if the Object is of type IXform we need to store it in the hierarchy (since we only need them for matrix animation right now)
		if (AbcImporterUtilities::IsType<Alembic::AbcGeom::IXform>(Parent.getMetaData()))
		{
			Hierarchy.AddHead(Alembic::AbcGeom::IXform(Parent, Alembic::Abc::kWrapExisting));
		}
		Parent = Parent.getParent();
	}
}

void AbcImporterUtilities::PropogateMatrixTransformationToSample(FAbcMeshSample* Sample, const FMatrix& Matrix)
{		
	for (FVector3f& Position : Sample->Vertices)
	{
		Position = (FVector4f)Matrix.TransformPosition((FVector)Position);
	}

	for (FVector3f& Velocity : Sample->Velocities)
	{
		Velocity = (FVector4f)Matrix.TransformVector((FVector)Velocity);
	}

	// TODO could make this a for loop and combine the transforms
	for (FVector3f& Normal : Sample->Normals)
	{
		Normal = (FVector4f)Matrix.TransformVector((FVector)Normal);
		Normal.Normalize();
	}

	for (FVector3f& TangentX : Sample->TangentX)
	{
		TangentX = (FVector4f)Matrix.TransformVector((FVector)TangentX);
		TangentX.Normalize();
	}

	for (FVector3f& TangentY : Sample->TangentY)
	{
		TangentY = (FVector4f)Matrix.TransformVector((FVector)TangentY);
		TangentY.Normalize();
	}
}

void AbcImporterUtilities::GenerateDeltaFrameDataMatrix(const TArray<FVector3f>& FrameVertexData, const TArray<FVector3f>& FrameNormalData, const TArray<FVector3f>& AverageVertexData, const TArray<FVector3f>& AverageNormalData,
	const int32 SampleIndex, const int32 AverageVertexOffset, const int32 AverageIndexOffset, const FVector& SamplePositionOffset, TArray64<float>& OutGeneratedMatrix, TArray64<float>& OutGeneratedNormalsMatrix)
{
	const int32 NumVertices = FrameVertexData.Num();
	const int64 VertexOffset = int64(SampleIndex) * AverageVertexData.Num() * 3;

	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		const int32 ComponentIndexOffset = (VertexIndex + AverageVertexOffset) * 3;
		const FVector AverageDifference = ((FVector)AverageVertexData[VertexIndex + AverageVertexOffset] + SamplePositionOffset) - (FVector)FrameVertexData[VertexIndex];

		OutGeneratedMatrix[VertexOffset + ComponentIndexOffset + 0] = static_cast<float>(AverageDifference.X);
		OutGeneratedMatrix[VertexOffset + ComponentIndexOffset + 1] = static_cast<float>(AverageDifference.Y);
		OutGeneratedMatrix[VertexOffset + ComponentIndexOffset + 2] = static_cast<float>(AverageDifference.Z);
	}

	const int32 NumIndices = FrameNormalData.Num();
	const int64 IndexOffset = int64(SampleIndex) * AverageNormalData.Num() * 3;

	for (int32 Index = 0; Index < NumIndices; ++Index)
	{
		const int32 ComponentIndexOffset = (Index + AverageIndexOffset) * 3;
		const FVector AverageNormal = (FVector)AverageNormalData[Index + AverageIndexOffset] - (FVector)FrameNormalData[Index];

		OutGeneratedNormalsMatrix[IndexOffset + ComponentIndexOffset + 0] = static_cast<float>(AverageNormal.X);
		OutGeneratedNormalsMatrix[IndexOffset + ComponentIndexOffset + 1] = static_cast<float>(AverageNormal.Y);
		OutGeneratedNormalsMatrix[IndexOffset + ComponentIndexOffset + 2] = static_cast<float>(AverageNormal.Z);
	}
}

void AbcImporterUtilities::GenerateCompressedMeshData(FCompressedAbcData& CompressedData, const uint32 NumUsedSingularValues, const uint32 NumSamples,
	const TArrayView64<float>& BasesMatrix, const TArrayView64<float>& NormalsBasesMatrix, const TArray64<float>& BasesWeights, const float SampleTimeStep, const float StartTime)
{
	FScopedSlowTask SlowTask(static_cast<float>(NumUsedSingularValues), FText::FromString(FString(TEXT("Generating bases"))));
	SlowTask.MakeDialog();

	// Allocate base sample data	
	CompressedData.BaseSamples.AddZeroed(NumUsedSingularValues);
	CompressedData.CurveValues.AddZeroed(NumUsedSingularValues);
	CompressedData.TimeValues.AddZeroed(NumUsedSingularValues);

	// Generate the bases data and weights
	for (uint32 BaseIndex = 0; BaseIndex < NumUsedSingularValues; ++BaseIndex)
	{
		FAbcMeshSample* Base = new FAbcMeshSample(*CompressedData.AverageSample);

		const int32 NumVertices = Base->Vertices.Num();
		const int32 NumMatrixRows = NumVertices * 3;
		const int64 BaseOffset = int64(BaseIndex) * NumMatrixRows;
		for (int32 Index = 0; Index < NumVertices; ++Index)
		{
			const int64 IndexOffset = BaseOffset + (Index * 3);
			FVector3f& BaseVertex = Base->Vertices[Index];

			BaseVertex.X -= BasesMatrix[IndexOffset + 0];
			BaseVertex.Y -= BasesMatrix[IndexOffset + 1];
			BaseVertex.Z -= BasesMatrix[IndexOffset + 2];
		}

		const int32 NumIndices = Base->Indices.Num();
		const int32 NumNormalsMatrixRows = NumIndices * 3;
		const int64 BaseIndexOffset = int64(BaseIndex) * NumNormalsMatrixRows;
		for (int32 Index = 0; Index < NumIndices; ++Index)
		{
			const int64 IndexOffset = BaseIndexOffset + (Index * 3);
			FVector3f& BaseNormal = Base->Normals[Index];

			BaseNormal.X -= NormalsBasesMatrix[IndexOffset + 0];
			BaseNormal.Y -= NormalsBasesMatrix[IndexOffset + 1];
			BaseNormal.Z -= NormalsBasesMatrix[IndexOffset + 2];
		}

		CompressedData.BaseSamples[BaseIndex] = Base;

		TArray<float>& CurveValues = CompressedData.CurveValues[BaseIndex];
		TArray<float>& TimeValues = CompressedData.TimeValues[BaseIndex];

		CurveValues.Reserve(NumSamples);
		TimeValues.Reserve(NumSamples);

		// Use original number of singular values to index into the array (otherwise we would be reading incorrect data if NumUsedSingularValues != the original number
		const int64 OriginalNumberOfSingularValues = BasesWeights.Num() / NumSamples;
		// Should be possible to rearrange the data so this can become a memcpy
		for (uint32 CurveSampleIndex = 0; CurveSampleIndex < NumSamples; ++CurveSampleIndex)
		{
			CurveValues.Add(BasesWeights[BaseIndex + (OriginalNumberOfSingularValues * CurveSampleIndex)]);
			TimeValues.Add(StartTime + (SampleTimeStep * static_cast<float>(CurveSampleIndex)));
		}

		SlowTask.EnterProgressFrame(1.f);
	}
}

void AbcImporterUtilities::CalculateNewStartAndEndFrameIndices(const float FrameStepRatio, int32& InOutStartFrameIndex, int32& InOutEndFrameIndex )
{
	// Using the calculated ratio we recompute the start/end frame indices
	InOutStartFrameIndex = FMath::Max(FMath::FloorToInt(static_cast<float>(InOutStartFrameIndex) * FrameStepRatio), 0);
	InOutEndFrameIndex = FMath::CeilToInt(static_cast<float>(InOutEndFrameIndex) * FrameStepRatio);
}

bool AbcImporterUtilities::AreVerticesEqual(const FSoftSkinVertex& V1, const FSoftSkinVertex& V2)
{
	if (FMath::Abs(V1.Position.X - V2.Position.X) > THRESH_POINTS_ARE_SAME
		|| FMath::Abs(V1.Position.Y - V2.Position.Y) > THRESH_POINTS_ARE_SAME
		|| FMath::Abs(V1.Position.Z - V2.Position.Z) > THRESH_POINTS_ARE_SAME)
	{
		return false;
	}

	// Set to 1 for now as we only import one UV set
	for (int32 UVIdx = 0; UVIdx < 1/*MAX_TEXCOORDS*/; ++UVIdx)
	{
		if (FMath::Abs(V1.UVs[UVIdx].X - V2.UVs[UVIdx].X) >(1.0f / 1024.0f))
			return false;

		if (FMath::Abs(V1.UVs[UVIdx].Y - V2.UVs[UVIdx].Y) > (1.0f / 1024.0f))
			return false;
	}

	FVector3f N1, N2;
	N1 = V1.TangentZ;
	N2 = V2.TangentZ;

	if (FMath::Abs(N1.X - N2.X) > THRESH_NORMALS_ARE_SAME || FMath::Abs(N1.Y - N2.Y) > THRESH_NORMALS_ARE_SAME || FMath::Abs(N1.Z - N2.Z) > THRESH_NORMALS_ARE_SAME)
	{
		return false;
	}

	return true;
}

void AbcImporterUtilities::ApplyConversion(FAbcMeshSample* InOutSample, const FAbcConversionSettings& InConversionSettings, const bool bShouldInverseBuffers)
{
	if ( InConversionSettings.bFlipV || InConversionSettings.bFlipU )
	{
		// Apply UV matrix to flip channels
		FMatrix2x2 UVMatrix( FScale2D(InConversionSettings.bFlipU ? -1.0f : 1.0f, InConversionSettings.bFlipV ? -1.0f : 1.0f) );
		FVector2f UVOffset(InConversionSettings.bFlipU ? 1.0f : 0.0f, InConversionSettings.bFlipV ? 1.0f : 0.0f);
				
		for (uint32 UVIndex = 0; UVIndex < InOutSample->NumUVSets; ++UVIndex)
		{
			for (FVector2f& UV : InOutSample->UVs[UVIndex])
			{
				UV = UVOffset + FVector2f(UVMatrix.TransformPoint(FVector2D(UV)));
			}
		}
	}
	
	// Calculate conversion matrix	
	const FMatrix Matrix = FScaleMatrix::Make(InConversionSettings.Scale) * FRotationMatrix::Make(FQuat::MakeFromEuler(InConversionSettings.Rotation));
	if (bShouldInverseBuffers && !Matrix.Equals(FMatrix::Identity))
	{
		// In case of negative determinant (e.g. negative scaling), invert the indice data
		if (Matrix.Determinant() < 0.0f)
		{
			Algo::Reverse(InOutSample->Indices);
			Algo::Reverse(InOutSample->Normals);
			Algo::Reverse(InOutSample->TangentX);
			Algo::Reverse(InOutSample->TangentY);
			for (uint32 UVIndex = 0; UVIndex < InOutSample->NumUVSets; ++UVIndex)
			{
				Algo::Reverse(InOutSample->UVs[UVIndex]);
			}
			Algo::Reverse(InOutSample->MaterialIndices);
			Algo::Reverse(InOutSample->SmoothingGroupIndices);
			Algo::Reverse(InOutSample->Colors);
		}
	}
}

bool AbcImporterUtilities::IsObjectVisible(const Alembic::Abc::IObject& Object, const Alembic::Abc::ISampleSelector FrameSelector)
{
	checkf(Object.valid(), TEXT("Invalid Object"));

	Alembic::Abc::ICompoundProperty CompoundProperty = Object.getProperties();
	Alembic::AbcGeom::IVisibilityProperty visibilityProperty;
	if (CompoundProperty.getPropertyHeader(Alembic::AbcGeom::kVisibilityPropertyName))
	{
		visibilityProperty = Alembic::AbcGeom::IVisibilityProperty(CompoundProperty,
			Alembic::AbcGeom::kVisibilityPropertyName);
	}

	Alembic::AbcGeom::ObjectVisibility visibilityValue = Alembic::AbcGeom::kVisibilityDeferred;
	if (visibilityProperty)
	{
		int8_t rawVisibilityValue;
		rawVisibilityValue = visibilityProperty.getValue(FrameSelector);
		visibilityValue = Alembic::AbcGeom::ObjectVisibility(rawVisibilityValue);
	}

	Alembic::Abc::IObject currentObject = Object;
	while (visibilityValue == Alembic::AbcGeom::kVisibilityDeferred)
	{
		// go up a level
		currentObject = currentObject.getParent();
		if (!currentObject)
		{
			return true;
		}

		CompoundProperty = currentObject.getProperties();
		if (CompoundProperty.getPropertyHeader(Alembic::AbcGeom::kVisibilityPropertyName))
		{
			visibilityProperty = Alembic::AbcGeom::IVisibilityProperty(CompoundProperty,
				Alembic::AbcGeom::kVisibilityPropertyName);
		}

		if (visibilityProperty && visibilityProperty.valid())
		{
			int8_t rawVisibilityValue;
			rawVisibilityValue = visibilityProperty.getValue(FrameSelector);
			visibilityValue = Alembic::AbcGeom::ObjectVisibility(rawVisibilityValue);
		}

		// At this point if we didn't find the visiblilty
		// property OR if the value was deferred we'll
		// continue up a level (so only if this object
		// says hidden OR explicitly says visible do we stop.
	}

	if (visibilityValue == Alembic::AbcGeom::kVisibilityHidden)
		return false;

	return true;
}

bool AbcImporterUtilities::IsObjectVisibilityConstant(const Alembic::Abc::IObject& Object)
{
	checkf(Object.valid(), TEXT("Invalid Object"));

	Alembic::Abc::ICompoundProperty CompoundProperty = Object.getProperties();
	Alembic::AbcGeom::IVisibilityProperty visibilityProperty;
	if (CompoundProperty.getPropertyHeader(Alembic::AbcGeom::kVisibilityPropertyName))
	{
		visibilityProperty = Alembic::AbcGeom::IVisibilityProperty(CompoundProperty,
			Alembic::AbcGeom::kVisibilityPropertyName);
	}

	bool bConstantVisibility = true;

	if (visibilityProperty)
	{
		bConstantVisibility = visibilityProperty.isConstant();
	}

	Alembic::Abc::IObject currentObject = Object;
	while (bConstantVisibility)
	{
		// go up a level
		currentObject = currentObject.getParent();
		if (!currentObject)
		{
			return bConstantVisibility;
		}

		CompoundProperty = currentObject.getProperties();
		if (CompoundProperty.getPropertyHeader(Alembic::AbcGeom::kVisibilityPropertyName))
		{
			visibilityProperty = Alembic::AbcGeom::IVisibilityProperty(CompoundProperty,
				Alembic::AbcGeom::kVisibilityPropertyName);
		}

		if (visibilityProperty && visibilityProperty.valid())
		{
			bConstantVisibility = visibilityProperty.isConstant();
		}
	}


	return bConstantVisibility;
}

FBoxSphereBounds AbcImporterUtilities::ExtractBounds(Alembic::Abc::IBox3dProperty InBoxBoundsProperty)
{
	FBoxSphereBounds::Builder BoundsBuilder;

	// Extract data only if the property is found	
	if (InBoxBoundsProperty.valid())
	{
		const int32 NumSamples = InBoxBoundsProperty.getNumSamples();
		for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
		{
			Alembic::Abc::Box3d BoundsSample;
			InBoxBoundsProperty.get(BoundsSample, SampleIndex);

			// Set up bounds from Alembic data format
			const Imath::V3d BoundSize = BoundsSample.size();
			const Imath::V3d BoundCenter = BoundsSample.center();
			const FBoxSphereBounds ConvertedBounds(FVector(BoundCenter.x, BoundCenter.y, BoundCenter.z), FVector(BoundSize.x  * 0.5f, BoundSize.y * 0.5f, BoundSize.z * 0.5f), (const float)BoundSize.length() * 0.5f);

			BoundsBuilder += ConvertedBounds;
		}
	}

	return BoundsBuilder;
}

void AbcImporterUtilities::ApplyConversion(FMatrix& InOutMatrix, const FAbcConversionSettings& InConversionSettings)
{
	// Calculate conversion matrix	
	const FMatrix ConversionMatrix = FScaleMatrix::Make(InConversionSettings.Scale) * FRotationMatrix::Make(FQuat::MakeFromEuler(InConversionSettings.Rotation));
	InOutMatrix = InOutMatrix * ConversionMatrix;
}

void AbcImporterUtilities::ApplyConversion(FBoxSphereBounds& InOutBounds, const FAbcConversionSettings& InConversionSettings)
{
	// Calculate conversion matrix	
	const FMatrix ConversionMatrix = FScaleMatrix::Make(InConversionSettings.Scale) * FRotationMatrix::Make(FQuat::MakeFromEuler(InConversionSettings.Rotation));
	if (!ConversionMatrix.Equals(FMatrix::Identity))
	{
		InOutBounds = InOutBounds.TransformBy(ConversionMatrix);
	}
}

void AbcImporterUtilities::ApplyConversion(TArray<FMatrix>& InOutMatrices, const FAbcConversionSettings& InConversionSettings)
{
	// Calculate conversion matrix	
	const FMatrix ConversionMatrix = FScaleMatrix::Make(InConversionSettings.Scale) * FRotationMatrix::Make(FQuat::MakeFromEuler(InConversionSettings.Rotation));
	if (!ConversionMatrix.Equals(FMatrix::Identity))
	{
		for (FMatrix& SampleMatrix : InOutMatrices)
		{			
			SampleMatrix = SampleMatrix * ConversionMatrix;
		}
	}
}

void AbcImporterUtilities::GeometryCacheDataForMeshSample(FGeometryCacheMeshData &OutMeshData, const FAbcMeshSample* MeshSample,
	const uint32 MaterialOffset, const float SecondsPerFrame, const bool bUseVelocitiesAsMotionVectors, const bool bStoreImportedVertexNumbers)
{
	OutMeshData.BoundingBox = MeshSample->Vertices.Num() ? FBox3f(MeshSample->Vertices) : FBox3f(EForceInit::ForceInit);
	if (MeshSample->Vertices.Num() == 0)
	{
		// Somehow the MeshSample can have Indices but no Vertices so just skip the whole thing if there are no vertices
		// Only the BoundingBox needs to be initialized to avoid asserts when rendering
		return;
	}

	// We currently always have everything except motion vectors
	// and tangents are auto-configure based on their presence in the mesh sample data
	const int32 NumNormals = MeshSample->Normals.Num();
	const bool bHasTangents = MeshSample->TangentX.Num() == NumNormals && MeshSample->TangentY.Num() == NumNormals;
	const bool bHasVelocities = bUseVelocitiesAsMotionVectors && (MeshSample->Velocities.Num() > 0);
	const bool bHasImportedVertexNumbers = (NumNormals > 0) && bStoreImportedVertexNumbers;

	OutMeshData.VertexInfo.bHasColor0 = true;
	OutMeshData.VertexInfo.bHasTangentX = bHasTangents;
	OutMeshData.VertexInfo.bHasTangentZ = NumNormals > 0;
	OutMeshData.VertexInfo.bHasUV0 = true;
	OutMeshData.VertexInfo.bHasMotionVectors = bHasVelocities;
	OutMeshData.VertexInfo.bHasImportedVertexNumbers = bHasImportedVertexNumbers;

	const int32 NumTriangles = MeshSample->Indices.Num() / 3;
	const uint32 NumSections = MeshSample->NumMaterials ? MeshSample->NumMaterials : 1;

	TArray<TArray<uint32>> SectionIndices;
	SectionIndices.AddDefaulted(NumSections);

	OutMeshData.Positions.AddZeroed(NumNormals);

	if (bHasVelocities)
	{
		OutMeshData.MotionVectors.AddZeroed(NumNormals);
	}

	if (bHasTangents)
	{
		OutMeshData.TangentsX.AddZeroed(NumNormals);
	}

	OutMeshData.TangentsZ.AddZeroed(NumNormals);
	OutMeshData.TextureCoordinates.AddZeroed(NumNormals);
	OutMeshData.Colors.AddZeroed(NumNormals);

	if (bHasImportedVertexNumbers)
	{
		OutMeshData.ImportedVertexNumbers.AddZeroed(NumNormals);
	}

	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
	{
		const int32 SectionIndex = MeshSample->MaterialIndices[TriangleIndex];
		TArray<uint32>& Section = SectionIndices[SectionIndex];

		for (int32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
		{
			const int32 CornerIndex = (TriangleIndex * 3) + VertexIndex;
			const int32 Index = MeshSample->Indices[CornerIndex];

			OutMeshData.Positions[CornerIndex] = MeshSample->Vertices[Index];

			if (bHasImportedVertexNumbers)
			{
				OutMeshData.ImportedVertexNumbers[CornerIndex] = Index;
			}

			if (bHasVelocities)
			{
				FVector3f MotionVector = MeshSample->Velocities[Index];
				MotionVector *= -1.f;
				MotionVector *= SecondsPerFrame; // Velocity is per seconds but we need per frame for motion vectors
				OutMeshData.MotionVectors[CornerIndex] = MotionVector;
			}

			OutMeshData.TangentsZ[CornerIndex] = MeshSample->Normals[CornerIndex];
			// store determinant of basis in w component of normal vector
			if (bHasTangents)
			{
				OutMeshData.TangentsX[CornerIndex] = MeshSample->TangentX[CornerIndex];
				OutMeshData.TangentsZ[CornerIndex].Vector.W = GetBasisDeterminantSignByte(MeshSample->TangentX[CornerIndex], MeshSample->TangentY[CornerIndex], MeshSample->Normals[CornerIndex]);
			}
			else
			{
				OutMeshData.TangentsZ[CornerIndex].Vector.W = 127;
			}
			OutMeshData.TextureCoordinates[CornerIndex] = MeshSample->UVs[0][CornerIndex];
			OutMeshData.Colors[CornerIndex] = MeshSample->Colors[CornerIndex].ToFColor(false);

			Section.Add(CornerIndex);
		}
	}

	uint32 NumMaterials = MaterialOffset;

	TArray<uint32>& Indices = OutMeshData.Indices;
	for (uint32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex, ++NumMaterials)
	{
		// Sometimes empty sections seem to be in the file, filter these out
		// as empty batches are not allowed by the geometry cache (They ultimately trigger checks in the renderer)
		// and it seems pretty nasty to filter them out post decode in-game
		if (!SectionIndices[SectionIndex].Num())
		{
			continue;
		}

		FGeometryCacheMeshBatchInfo BatchInfo;
		BatchInfo.StartIndex = Indices.Num();
		BatchInfo.MaterialIndex = NumMaterials;

		BatchInfo.NumTriangles = SectionIndices[SectionIndex].Num() / 3;
		Indices.Append(SectionIndices[SectionIndex]);
		OutMeshData.BatchesInfo.Add(BatchInfo);
	}
}

void AbcImporterUtilities::MergePolyMeshesToMeshData(int32 FrameIndex, int32 FrameStart, float SecondsPerFrame, bool bUseVelocitiesAsMotionVectors,
	const TArray<FAbcPolyMesh*>& PolyMeshes, 
	const TArray<int32> LookupMaterialSlot,
	float& FrameTime,
	FGeometryCacheMeshData& MeshData, int32& PreviousNumVertices, bool& bConstantTopology, bool bStoreImportedVertexNumbers)
{
	FAbcMeshSample MergedSample;

	for (FAbcPolyMesh* PolyMesh : PolyMeshes)
	{
		if (PolyMesh->bShouldImport)
		{
			const int32 Offset = MergedSample.MaterialIndices.Num();
			bConstantTopology = bConstantTopology && PolyMesh->bConstantTopology;

			// The caller expects this value to be calculated regardless of mesh visibility
			FrameTime = PolyMesh->GetTimeForFrameIndex(FrameIndex);

			if (PolyMesh->GetVisibility(FrameIndex))
			{
				const FAbcMeshSample* Sample = PolyMesh->GetSample(FrameIndex);
				AbcImporterUtilities::AppendMeshSample(&MergedSample, Sample);
			
				for (int32 Index = Offset; Index < MergedSample.MaterialIndices.Num(); ++Index)
				{
					int32 MaterialID = MergedSample.MaterialIndices[Index];
					MergedSample.MaterialIndices[Index] = LookupMaterialSlot[MaterialID];
				}
			}
			else
			{
				MergedSample.NumMaterials += PolyMesh->FaceSetNames.Num();

				// If this mesh is invisible on this frame then the topology is not constant!
				bConstantTopology = false;
			}

		}
	}

	if (FrameIndex > FrameStart)						
	{
		bConstantTopology &= (PreviousNumVertices == MergedSample.Vertices.Num());
	}
	PreviousNumVertices = MergedSample.Vertices.Num();

	// For some reason, merged meshes (especially with varying topologies) can end up with an invalid number of material indices.
	// Here, the material indices is filled up with 0 (the first material) to match the number of expected triangles so that the
	// generated geometry cache mesh data from GeometryCacheDataForMeshSample is complete.
	const int32 NumTriangles = MergedSample.Indices.Num() / 3;
	const int32 NumMatIndices = MergedSample.MaterialIndices.Num();
	if (!ensureMsgf(NumMatIndices >= NumTriangles, TEXT("Patching merged meshes (%d) since there are fewer material indices than triangles (%d < %d)."), PolyMeshes.Num(), NumMatIndices, NumTriangles))
	{
		const int32 Delta = NumTriangles - NumMatIndices;
		MergedSample.MaterialIndices.AddZeroed(Delta);
	}

	// Generate the mesh data for this sample
	AbcImporterUtilities::GeometryCacheDataForMeshSample(MeshData, &MergedSample, 0, SecondsPerFrame, bUseVelocitiesAsMotionVectors, bStoreImportedVertexNumbers);
}

UMaterialInterface* AbcImporterUtilities::RetrieveMaterial(FAbcFile& AbcFile, const FString& MaterialName, UObject* InParent, EObjectFlags Flags)
{
	UMaterialInterface* Material = nullptr;
	UMaterialInterface** CachedMaterial = AbcFile.GetMaterialByName(MaterialName);
	if (CachedMaterial)
	{
		Material = *CachedMaterial;

		// Setup package name next to InParent
		FString NewPackageName = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetPathName()) + TEXT("/") + MaterialName;
		NewPackageName = UPackageTools::SanitizePackageName(NewPackageName);

		// Material could have been deleted if we're overriding/reimporting an asset
		if (Material->IsValidLowLevel())
		{
			if (Material->GetOuter() == GetTransientPackage())
			{
				UPackage* Package = CreatePackage(*NewPackageName);
				UObject* ExistingObject = FindObject<UObject>(Package, *MaterialName);
				if (ExistingObject)
				{
					return Cast<UMaterial>(ExistingObject);
				}

				Material->Rename(*MaterialName, Package);				
				Material->SetFlags(Flags);
				FAssetRegistryModule::AssetCreated(Material);
			}
		}
		else
		{
			// #ueent_todo: Revisit this code path
			// In this case recreate the material
			UPackage* Package = CreatePackage(*NewPackageName);
			UObject* ExistingObject = FindObject<UObject>(Package, *MaterialName);
			if (ExistingObject)
			{
				return Cast<UMaterial>(ExistingObject);
			}

			Material = NewObject<UMaterial>(Package, *MaterialName);
			Material->SetFlags(Flags);
			FAssetRegistryModule::AssetCreated(Material);
		}
	}
	else
	{
		Material = UMaterial::GetDefaultMaterial(MD_Surface);
		check(Material);
	}

	return Material;
}

#undef LOCTEXT_NAMESPACE // "AbcImporterUtilities"
