// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h" // IWYU pragma: keep

#if PLATFORM_WINDOWS
#include "Math/Matrix.h"
#include "UObject/ObjectMacros.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <Alembic/AbcCoreAbstract/TimeSampling.h>
#include <Alembic/Abc/All.h>
#include <Alembic/AbcGeom/All.h>
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

class IMeshUtilities;
namespace Alembic::Abc::v12 { class IObject; }
namespace Alembic::AbcCoreAbstract::v12 { class MetaData; }
namespace Alembic::AbcGeom::v12 { class IPolyMeshSchema; }
struct FAbcConversionSettings;
struct FGeometryCacheMeshData;
struct FSoftSkinVertex;
template <class ElementType> class TDoubleLinkedList;

class FAbcFile;
class FAbcPolyMesh;
class UMaterialInterface;
struct FAbcMeshSample;
struct FCompressedAbcData;

enum class ESampleReadFlags : uint8
{
	Default = 0,
	Positions = 1 << 1,
	Indices = 1 << 2,
	UVs = 1 << 3,
	Normals = 1 << 4,
	Colors = 1 << 5,
	MaterialIndices = 1 << 6,
	Velocities = 1 << 7
};
ENUM_CLASS_FLAGS(ESampleReadFlags);

namespace AbcImporterUtilities
{
	/** Templated function to check whether or not an object is of a certain type */
	template<typename T> bool IsType(const Alembic::Abc::MetaData& MetaData)
	{
		return T::matches(MetaData);
	}

	/**
	* ConvertAlembicMatrix, converts Abc(Alembic) matrix to UnrealEngine matrix format
	*
	* @param AbcMatrix - Alembic style matrix
	* @return FMatrix
	*/
	FMatrix ConvertAlembicMatrix(const Alembic::Abc::M44d& AbcMatrix);

	uint32 GenerateMaterialIndicesFromFaceSets(Alembic::AbcGeom::IPolyMeshSchema &Schema, const Alembic::Abc::ISampleSelector FrameSelector, TArray<int32> &MaterialIndicesOut);

	void RetrieveFaceSetNames(Alembic::AbcGeom::IPolyMeshSchema &Schema, TArray<FString>& NamesOut);

	template<typename T, typename U> bool RetrieveTypedAbcData(T InSampleDataPtr, TArray<U>& OutDataArray )
	{
		if (!InSampleDataPtr)
		{
			return false;
		}

		// Allocate required memory for the OutData
		const int32 NumEntries = InSampleDataPtr->size();
		bool bSuccess = false; 
		
		if (NumEntries)
		{
			OutDataArray.AddZeroed(NumEntries);
			auto DataPtr = InSampleDataPtr->get();
			auto OutDataPtr = &OutDataArray[0];

			// Ensure that the destination and source data size corresponds (otherwise we will end up with an invalid memcpy and means we have a type mismatch)
			if (sizeof(DataPtr[0]) == sizeof(OutDataArray[0]))
			{
				FMemory::Memcpy(OutDataPtr, DataPtr, sizeof(U) * NumEntries);
				bSuccess = true;
			}
		}	

		return bSuccess;
	}

	/** Expands the given vertex attribute array to not be indexed */
	template<typename T> void ExpandVertexAttributeArray(const TArray<uint32>& InIndices, TArray<T>& InOutArray)
	{
		const int32 NumIndices = InIndices.Num();		
		TArray<T> NewArray;
		NewArray.Reserve(NumIndices);

		for (const uint32 Index : InIndices)
		{
			NewArray.Add(InOutArray[Index]);
		}

		InOutArray = NewArray;
	}

	/** Triangulates the given index buffer (assuming incoming data is quads or quad/triangle mix) */
	void TriangulateIndexBuffer(const TArray<uint32>& InFaceCounts, TArray<uint32>& InOutIndices);

	/** Triangulates the given (non-indexed) vertex attribute data buffer (assuming incoming data is quads or quad/triangle mix) */
	template<typename T> void TriangulateVertexAttributeBuffer(const TArray<uint32>& InFaceCounts, TArray<T>& InOutData)
	{
		check(InFaceCounts.Num() > 0);
		check(InOutData.Num() > 0);

		TArray<T> NewData;
		NewData.Reserve(InFaceCounts.Num() * 4);

		uint32 Index = 0;
		for (const uint32 NumIndicesForFace : InFaceCounts)
		{
			if (NumIndicesForFace > 3)
			{
				// Triangle 0
				NewData.Add(InOutData[Index]);
				NewData.Add(InOutData[Index + 1]);
				NewData.Add(InOutData[Index + 3]);


				// Triangle 1
				NewData.Add(InOutData[Index + 3]);
				NewData.Add(InOutData[Index + 1]);
				NewData.Add(InOutData[Index + 2]);
			}
			else
			{
				NewData.Add(InOutData[Index]);
				NewData.Add(InOutData[Index + 1]);
				NewData.Add(InOutData[Index + 2]);
			}

			Index += NumIndicesForFace;
		}

		// Set new data
		InOutData = NewData;
	}

	template<typename T> void ExpandPrimitiveAttributeArray(const TArray<uint32>& InFaceCounts, TArray<T>& InOutArray)
	{
		int32 NumIndices = 0;
		for (const uint32 NumIndicesForFace : InFaceCounts)
		{
			NumIndices += NumIndicesForFace;
		}

		TArray<T> NewArray;
		NewArray.Reserve(NumIndices);

		int32 PrimitiveIndex = 0;
		for (const uint32 NumIndicesForFace : InFaceCounts)
		{
			T data = InOutArray[PrimitiveIndex];
			if (NumIndicesForFace > 3)
			{
				// Triangle 0
				NewArray.Add(data);
				NewArray.Add(data);
				NewArray.Add(data);

				// Triangle 1
				NewArray.Add(data);
				NewArray.Add(data);
				NewArray.Add(data);
			}
			else
			{
				NewArray.Add(data);
				NewArray.Add(data);
				NewArray.Add(data);
			}

			PrimitiveIndex++;
		}

		InOutArray = NewArray;
	}

	template<typename T> void ProcessVertexAttributeArray(const TArray<uint32>& InIndices, const TArray<uint32>& InFaceCounts, const bool bTriangulation, const uint32 NumVertices, TArray<T>& InOutArray)
	{
		// Expand using the vertex indices (if num entries == num vertices)
		if (InOutArray.Num() != InIndices.Num() && InOutArray.Num() == NumVertices)
		{
			ExpandVertexAttributeArray(InIndices, InOutArray);
		}
		// Expand using the primitive indices (if num entries == num faces)
		else if (InOutArray.Num() != InIndices.Num() && InOutArray.Num() == InFaceCounts.Num())
		{
			ExpandPrimitiveAttributeArray(InFaceCounts, InOutArray);
		}
		// Otherwise the attributes are stored per face, so triangulate if the faces contain quads
		else if (bTriangulation)
		{
			TriangulateVertexAttributeBuffer(InFaceCounts, InOutArray);
		}
	}
	
	/** Triangulates material indices according to the face counts (quads will have to be split up into two faces / material indices)*/
	void TriangulateMaterialIndices(const TArray<uint32>& InFaceCounts, TArray<int32>& InOutData);

	template<typename T>
	Alembic::Abc::ISampleSelector GenerateAlembicSampleSelector(const T SelectionValue)
	{
		Alembic::Abc::ISampleSelector Selector(SelectionValue);
		return Selector;
	}

	/** Generates the data for an FAbcMeshSample instance given an Alembic PolyMesh schema and frame index */
	FAbcMeshSample* GenerateAbcMeshSampleForFrame(const Alembic::AbcGeom::IPolyMeshSchema& Schema, const Alembic::Abc::ISampleSelector FrameSelector, const ESampleReadFlags ReadFlags, const bool bFirstFrame = false);

	/** Generates a sample read bit mask to reduce unnecessary reads / memory allocations */
	ESampleReadFlags GenerateAbcMeshSampleReadFlags(const Alembic::AbcGeom::IPolyMeshSchema& Schema);

	/** Generated smoothing groups based on the given face normals, will compare angle between adjacent normals to determine whether or not an edge is hard/soft
		and calculates the smoothing group information with the edge data */
	void GenerateSmoothingGroups(TMultiMap<uint32, uint32> &TouchingFaces, const TArray<FVector3f>& FaceNormals,
		TArray<uint32>& FaceSmoothingGroups, uint32& HighestSmoothingGroup, const float HardAngleDotThreshold);
	
	/** Generates AbcMeshSample with given parameters and schema */
	bool GenerateAbcMeshSampleDataForFrame(const Alembic::AbcGeom::IPolyMeshSchema &Schema, const Alembic::Abc::ISampleSelector FrameSelector, FAbcMeshSample* &Sample, const ESampleReadFlags ReadFlags, const bool bFirstFrame );

	/** Read out texture coordinate data from Alembic GeometryParameter */
	void ReadUVSetData(Alembic::AbcGeom::IV2fGeomParam &UVCoordinateParameter, const Alembic::Abc::ISampleSelector FrameSelector, TArray<FVector2f>& OutUVs, const TArray<uint32>& MeshIndices, const bool bNeedsTriangulation, const TArray<uint32>& FaceCounts, const int32 NumVertices);

	void GenerateSmoothingGroupsIndices(FAbcMeshSample* MeshSample, float HardEdgeAngleThreshold);

	void CalculateNormals(FAbcMeshSample* Sample);

	void CalculateSmoothNormals(FAbcMeshSample* Sample);

	void CalculateNormalsWithSmoothingGroups(FAbcMeshSample* Sample, const TArray<uint32>& SmoothingMasks, const uint32 NumSmoothingGroups);
	void CalculateNormalsWithSampleData(FAbcMeshSample* Sample, const FAbcMeshSample* SourceSample);

	void ComputeTangents(FAbcMeshSample* Sample, bool bIgnoreDegenerateTriangles, IMeshUtilities& MeshUtilities);

	template<typename T> float RetrieveTimeForFrame(T& Schema, const uint32 FrameIndex)
	{
		checkf(Schema.valid(), TEXT("Invalid Schema"));
		Alembic::AbcCoreAbstract::TimeSamplingPtr TimeSampler = Schema.getTimeSampling();
		const double Time = TimeSampler->getSampleTime(FrameIndex);
		return (float)Time;
	}

	template<typename T> void GetMinAndMaxTime(T& Schema, float& MinTime, float& MaxTime)
	{
		checkf(Schema.valid(), TEXT("Invalid Schema"));
		Alembic::AbcCoreAbstract::TimeSamplingPtr TimeSampler = Schema.getTimeSampling();
		MinTime = (float)TimeSampler->getSampleTime(0);
		MaxTime = (float)TimeSampler->getSampleTime(Schema.getNumSamples() - 1);
	}

	template<typename T> void GetStartTimeAndFrame(T& Schema, float& StartTime, uint32& StartFrame)
	{
		checkf(Schema.valid(), TEXT("Invalid Schema"));
		Alembic::AbcCoreAbstract::TimeSamplingPtr TimeSampler = Schema.getTimeSampling();

		StartTime = (float)TimeSampler->getSampleTime(0);
		Alembic::AbcCoreAbstract::TimeSamplingType SamplingType = TimeSampler->getTimeSamplingType();
		// We know the seconds per frame, so if we take the time for the first stored sample we can work out how many 'empty' frames come before it
		// Ensure that the start frame is never lower that 0
		StartFrame = FMath::Max<int32>( FMath::CeilToInt(StartTime / (float)SamplingType.getTimePerCycle()), 0 );
	}

	template<typename T> void GetStartTimeAndFrame(T& Schema, float& StartTime, int32& StartFrame)
	{
		checkf(Schema.valid(), TEXT("Invalid Schema"));
		Alembic::AbcCoreAbstract::TimeSamplingPtr TimeSampler = Schema.getTimeSampling();

		StartTime = (float)TimeSampler->getSampleTime(0);
		Alembic::AbcCoreAbstract::TimeSamplingType SamplingType = TimeSampler->getTimeSamplingType();
		// We know the seconds per frame, so if we take the time for the first stored sample we can work out how many 'empty' frames come before it
		// Ensure that the start frame is never lower that 0
		StartFrame = FMath::CeilToInt(StartTime / (float)SamplingType.getTimePerCycle());
	}
	
	FAbcMeshSample* MergeMeshSamples(const TArray<const FAbcMeshSample*>& Samples);

	FAbcMeshSample* MergeMeshSamples(FAbcMeshSample* MeshSampleOne, FAbcMeshSample* MeshSampleTwo);
			
	void AppendMeshSample(FAbcMeshSample* MeshSampleOne, const FAbcMeshSample* MeshSampleTwo);
			
	void GetHierarchyForObject(const Alembic::Abc::IObject& Object, TDoubleLinkedList<Alembic::AbcGeom::IXform>& Hierarchy);

	void PropogateMatrixTransformationToSample(FAbcMeshSample* Sample, const FMatrix& Matrix);

	/** Generates the delta frame data for the given average and frame vertex data */
	void GenerateDeltaFrameDataMatrix(const TArray<FVector3f>& FrameVertexData, const TArray<FVector3f>& FrameNormalData, const TArray<FVector3f>& AverageVertexData, const TArray<FVector3f>& AverageNormalData,
		const int32 SampleIndex, const int32 AverageVertexOffset, const int32 AverageIndexOffset, const FVector& SamplePositionOffset, TArray64<float>& OutGeneratedMatrix, TArray64<float>& OutGeneratedNormalsMatrix);

	/** Populates compressed data structure from the result PCA compression bases and weights */
	void GenerateCompressedMeshData(FCompressedAbcData& CompressedData, const uint32 NumUsedSingularValues, const uint32 NumSamples,
		const TArrayView64<float>& BasesMatrix, const TArrayView64<float>& NormalsBasesMatrix, const TArray64<float>& BasesWeights, const float SampleTimeStep, const float StartTime);

	void CalculateNewStartAndEndFrameIndices(const float FrameStepRatio, int32& InOutStartFrameIndex, int32& InOutEndFrameIndex );

	bool AreVerticesEqual(const FSoftSkinVertex& V1, const FSoftSkinVertex& V2);

	/** Applies user/preset conversion to the given sample */
	void ApplyConversion(FAbcMeshSample* InOutSample, const FAbcConversionSettings& InConversionSettings, const bool bShouldInverseBuffers);

	/** Applies user/preset conversion to the given matrices */
	void ApplyConversion(TArray<FMatrix>& InOutMatrices, const FAbcConversionSettings& InConversionSettings);

	/** Applies user/preset conversion to the given matrices */
	void ApplyConversion(FMatrix& InOutMatrix, const FAbcConversionSettings& InConversionSettings);

	/** Extracts the bounding box from the given alembic property (initialised to zero if the property is invalid) */
	FBoxSphereBounds ExtractBounds(Alembic::Abc::IBox3dProperty InBoxBoundsProperty);
	
	/** Applies user/preset conversion to the given BoxSphereBounds */
	void ApplyConversion(FBoxSphereBounds& InOutBounds, const FAbcConversionSettings& InConversionSettings);	
	/** Returns whether or not the given Object is visible according at the retrieved frame using SampleSelector (this includes parent Objects) */
	bool IsObjectVisible(const Alembic::Abc::IObject& Object, const Alembic::Abc::ISampleSelector FrameSelector);
	/** Returns whether or not the Objects visibility property is constant across the entire sequence (this includes parent Objects) */
	bool IsObjectVisibilityConstant(const Alembic::Abc::IObject& Object);

	/** Generates and populates a FGeometryCacheMeshData instance from and for the given mesh sample */
	void GeometryCacheDataForMeshSample(FGeometryCacheMeshData &OutMeshData, const FAbcMeshSample* MeshSample, const uint32 MaterialOffset, const float SecondsPerFrame, const bool bUseVelocitiesAsMotionVectors, const bool bStoreImportedVertexNumbers);

	/**
	 * Merges the given PolyMeshes at the given FrameIndex into a GeometryCacheMeshData
	 *
	 * @param FrameIndex			The frame index to merge the PolyMeshes at
	 * @param FrameStart			The starting frame number of the range being processed
	 * @param SecondsPerFrame		The time step of the Abc file
	 * @param bUseVelocitiesAsMotionVectors		Converts the AbcPolyMesh velocities to motion vectors
	 * @param PolyMeshes			The PolyMeshes to merge, which will be sampled at FrameIndex
	 * @param LookupMaterialSlot	Mapping from original face set index (index in flattened list of facesets to import) to material slot (one for each unique faceset name)
	 * @param FrameTime				The frame time that corresponds to FrameIndex
	 * @param MeshData				The GeometryCacheMeshData where to output the merged PolyMeshes
	 * @param PreviousNumVertices	The number of vertices in the merged PolyMeshes, used to determine if its topology is constant between 2 frames
	 * @param bConstantTopology		Flag to indicate if the merged PolyMeshes has constant topology
	 * @param bStoreImportedVertexNumbers Set to true when we want to store the original dcc vertex numbers for each vertex.
	 */
	void MergePolyMeshesToMeshData(int32 FrameIndex, int32 FrameStart, float SecondsPerFrame, bool bUseVelocitiesAsMotionVectors,
		const TArray<FAbcPolyMesh*>& PolyMeshes, 
		const TArray<int32> LookupMaterialSlot,
		float& FrameTime,
		FGeometryCacheMeshData& MeshData, int32& PreviousNumVertices, bool& bConstantTopology, bool bStoreImportedVertexNumbers);

	/** Retrieves a material from an AbcFile according to the given name and resaves it into the parent package */
	UMaterialInterface* RetrieveMaterial(FAbcFile& AbcFile, const FString& MaterialName, UObject* InParent, EObjectFlags Flags);
}
