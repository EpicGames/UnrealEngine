// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcImporter.h"

#include "AbcAssetImportData.h"
#include "AbcFile.h"
#include "AbcImportLogger.h"
#include "AbcImportUtilities.h"
#include "AbcPolyMesh.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Async/ParallelFor.h"
#include "BoneWeights.h"
#include "ComponentReregisterContext.h"
#include "Editor.h"
#include "EigenHelper.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "FeedbackContextEditor.h"
#include "GeometryCache.h"
#include "GeometryCacheCodecV1.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrackStreamable.h"
#include "Logging/TokenizedMessage.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MeshBudgetProjectSettings.h"
#include "MeshUtilities.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "RenderMath.h"
#include "Rendering/SkeletalMeshModel.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/MetaData.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

THIRD_PARTY_INCLUDES_START
#include "Animation/AnimData/AnimDataModel.h"
#include <Alembic/AbcCoreFactory/All.h>
#include "Animation/AnimData/IAnimationDataController.h"
#include <Alembic/AbcCoreOgawa/All.h>
THIRD_PARTY_INCLUDES_END

#define LOCTEXT_NAMESPACE "AbcImporter"

DEFINE_LOG_CATEGORY_STATIC(LogAbcImporter, Verbose, All);

#define OBJECT_TYPE_SWITCH(a, b, c) if (AbcImporterUtilities::IsType<a>(ObjectMetaData)) { \
	a TypedObject = a(b, Alembic::Abc::kWrapExisting); \
	ParseAbcObject<a>(TypedObject, c); bHandled = true; }

#define PRINT_UNIQUE_VERTICES 0

static const FString NoFaceSetNameStr(TEXT("NoFaceSetName"));
static const FName NoFaceSetName(TEXT("NoFaceSetName"));

FAbcImporter::FAbcImporter()
	: ImportSettings(nullptr), AbcFile(nullptr)
{

}

FAbcImporter::~FAbcImporter()
{
	delete AbcFile;
}

void FAbcImporter::UpdateAssetImportData(UAbcAssetImportData* AssetImportData)
{
	AssetImportData->TrackNames.Empty();
	const TArray<FAbcPolyMesh*>& PolyMeshes = AbcFile->GetPolyMeshes();
	for (const FAbcPolyMesh* PolyMesh : PolyMeshes)
	{
		if (PolyMesh->bShouldImport)
		{
			AssetImportData->TrackNames.Add(PolyMesh->GetName());
		}
	}

	AssetImportData->SamplingSettings = ImportSettings->SamplingSettings;
	AssetImportData->NormalGenerationSettings = ImportSettings->NormalGenerationSettings;
	AssetImportData->CompressionSettings = ImportSettings->CompressionSettings;
	AssetImportData->StaticMeshSettings = ImportSettings->StaticMeshSettings;
	AssetImportData->GeometryCacheSettings = ImportSettings->GeometryCacheSettings;
	AssetImportData->ConversionSettings = ImportSettings->ConversionSettings;
}

void FAbcImporter::RetrieveAssetImportData(UAbcAssetImportData* AssetImportData)
{
	bool bAnySetForImport = false;

	for (FAbcPolyMesh* PolyMesh : AbcFile->GetPolyMeshes())
	{
		if (AssetImportData->TrackNames.Contains(PolyMesh->GetName()))
		{
			PolyMesh->bShouldImport = true;
			bAnySetForImport = true;
		}		
		else
		{
			PolyMesh->bShouldImport = false;
		}
	}

	// If none were set to import, set all of them to import (probably different scene/setup)
	if (!bAnySetForImport)
	{
		for (FAbcPolyMesh* PolyMesh : AbcFile->GetPolyMeshes())
		{
			PolyMesh->bShouldImport = true;
		}
	}
}

const EAbcImportError FAbcImporter::OpenAbcFileForImport(const FString InFilePath)
{
	AbcFile = new FAbcFile(InFilePath);
	return AbcFile->Open();
}

const EAbcImportError FAbcImporter::ImportTrackData(const int32 InNumThreads, UAbcImportSettings* InImportSettings)
{
	ImportSettings = InImportSettings;
	ImportSettings->NumThreads = InNumThreads;
	EAbcImportError Error = AbcFile->Import(ImportSettings);	

	return Error;
}

template<typename T>
T* FAbcImporter::CreateObjectInstance(UObject*& InParent, const FString& ObjectName, const EObjectFlags Flags, bool& bObjectAlreadyExists)
{
	// Parent package to place new asset
	UPackage* Package = nullptr;
	FString NewPackageName;
	bObjectAlreadyExists = false;

	// Setup package name and create one accordingly
	NewPackageName = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetPathName()) + TEXT("/") + ObjectName;
	NewPackageName = UPackageTools::SanitizePackageName(NewPackageName);
	Package = CreatePackage(*NewPackageName);

	const FString SanitizedObjectName = ObjectTools::SanitizeObjectName(ObjectName);

	T* ExistingTypedObject = FindObject<T>(Package, *SanitizedObjectName);
	UObject* ExistingObject = FindObject<UObject>(Package, *SanitizedObjectName);

	if (ExistingTypedObject != nullptr)
	{
		ExistingTypedObject->PreEditChange(nullptr);
		bObjectAlreadyExists = true;
		return ExistingTypedObject;
	}
	else if (ExistingObject != nullptr)
	{
		// Replacing an object.  Here we go!
		// Delete the existing object
		const bool bDeleteSucceeded = ObjectTools::DeleteSingleObject(ExistingObject);

		if (bDeleteSucceeded)
		{
			// Force GC so we can cleanly create a new asset (and not do an 'in place' replacement)
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

			// Create a package for each mesh
			Package = CreatePackage(*NewPackageName);
			InParent = Package;
		}
		else
		{
			// failed to delete
			return nullptr;
		}
	}

	return NewObject<T>(Package, FName(*SanitizedObjectName), Flags | RF_Public);
}

UStaticMesh* FAbcImporter::CreateStaticMeshFromSample(UObject* InParent, const FString& Name, EObjectFlags Flags, const TArray<FString>& UniqueFaceSetNames, const TArray<int32>& LookupMaterialSlot, const FAbcMeshSample* Sample)
{
	bool bObjectAlreadyExists = false;
	UStaticMesh* StaticMesh = CreateObjectInstance<UStaticMesh>(InParent, Name, Flags, bObjectAlreadyExists);

	// Only import data if a valid object was created
	if (StaticMesh)
	{
		// Add the first LOD, we only support one
		int32 LODIndex = 0;
		StaticMesh->AddSourceModel();
		FMeshDescription* MeshDescription = StaticMesh->CreateMeshDescription(LODIndex);
		// Generate a new lighting GUID (so its unique)
		StaticMesh->SetLightingGuid();

		// Set it to use textured lightmaps. Note that Build Lighting will do the error-checking (texcoord index exists for all LODs, etc).
		StaticMesh->SetLightMapResolution(64);
		StaticMesh->SetLightMapCoordinateIndex(1);

		// Material setup, since there isn't much material information in the Alembic file, 
		UMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
		check(DefaultMaterial);

		// Material list
		StaticMesh->GetStaticMaterials().Empty();

		// Build the material slots : one for each faceset
		for (const FString& FaceSetName : UniqueFaceSetNames)
		{
			UMaterialInterface* Material = AbcImporterUtilities::RetrieveMaterial(*AbcFile, FaceSetName, InParent, Flags);

			if (Material == nullptr)
			{
				Material = DefaultMaterial;
			}
			else if (Material != DefaultMaterial)
			{
				Material->PostEditChange();
			}

			FName MaterialName(*FaceSetName);

			StaticMesh->GetStaticMaterials().Add(FStaticMaterial(Material, MaterialName, MaterialName));
		}

		GenerateMeshDescriptionFromSample(UniqueFaceSetNames, LookupMaterialSlot, Sample, MeshDescription);

		// Get the first LOD for filling it up with geometry, only support one LOD
		FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LODIndex);
		// Set build settings for the static mesh
		SrcModel.BuildSettings.bRecomputeNormals = false;
		SrcModel.BuildSettings.bRecomputeTangents = false;
		SrcModel.BuildSettings.bUseMikkTSpace = false;
		// Generate Lightmaps uvs (no support for importing right now)
		SrcModel.BuildSettings.bGenerateLightmapUVs = ImportSettings->StaticMeshSettings.bGenerateLightmapUVs;
		// Set lightmap UV index to 1 since we currently only import one set of UVs from the Alembic Data file
		SrcModel.BuildSettings.DstLightmapIndex = 1;

		// Store the mesh description
		StaticMesh->CommitMeshDescription(LODIndex);

		//Set the Imported version before calling the build
		StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);

		FMeshBudgetProjectSettingsUtils::SetLodGroupForStaticMesh(StaticMesh);

		// Build the static mesh (using the build setting etc.) this generates correct tangents using the extracting smoothing group along with the imported Normals data
		StaticMesh->Build(false);

		// No collision generation for now
		StaticMesh->CreateBodySetup();
	}

	return StaticMesh;
}

const TArray<UStaticMesh*> FAbcImporter::ImportAsStaticMesh(UObject* InParent, EObjectFlags Flags)
{
	checkf(AbcFile->GetNumPolyMeshes() > 0, TEXT("No poly meshes found"));

	TArray<UStaticMesh*> ImportedStaticMeshes;
	const FAbcStaticMeshSettings& StaticMeshSettings = ImportSettings->StaticMeshSettings;

	TFunction<void(int32, FAbcFile*)> Func = [this, &ImportedStaticMeshes, StaticMeshSettings, InParent, Flags](int32 FrameIndex, FAbcFile* InFile)
	{
		const TArray<FAbcPolyMesh*>& PolyMeshes = AbcFile->GetPolyMeshes();
		if (StaticMeshSettings.bMergeMeshes) 
		{
			// Merge all meshes in the Alembic cache to one single static mesh
			TArray<const FAbcMeshSample*> SamplesToMerge;
			for (const FAbcPolyMesh* PolyMesh : PolyMeshes)
			{
				if (PolyMesh->bShouldImport)
				{
					const FAbcMeshSample* Sample = PolyMesh->GetSample(FrameIndex);
					SamplesToMerge.Add(Sample);
				}
			}

			if (SamplesToMerge.Num())
			{
				FAbcMeshSample* MergedSample = AbcImporterUtilities::MergeMeshSamples(SamplesToMerge);
			
				UStaticMesh* StaticMesh = CreateStaticMeshFromSample(InParent, 
					InParent != GetTransientPackage() ? FPaths::GetBaseFilename(InParent->GetName()) : (FPaths::GetBaseFilename(AbcFile->GetFilePath()) + "_" + FGuid::NewGuid().ToString()),
					Flags, 
					AbcFile->GetUniqueFaceSetNames(),
					AbcFile->GetLookupMaterialSlot(),
					MergedSample);

				if (StaticMesh)
				{
					ImportedStaticMeshes.Add(StaticMesh);
				}

				delete MergedSample; // Delete this temporary mesh
			}
		}
		else
		{
			for (const FAbcPolyMesh* PolyMesh : PolyMeshes)
			{
				const FAbcMeshSample* Sample = PolyMesh->GetSample(FrameIndex);
				if (PolyMesh->bShouldImport && Sample)
				{
					TArray<int32> LookupMaterialSlot;
					for (int32 MaterialSlotIndex = 0; MaterialSlotIndex < PolyMesh->FaceSetNames.Num(); ++MaterialSlotIndex)
					{
						LookupMaterialSlot.Add(MaterialSlotIndex);
					}

					// Setup static mesh instance
					UStaticMesh* StaticMesh = CreateStaticMeshFromSample(InParent, 
						InParent != GetTransientPackage() ? PolyMesh->GetName() : PolyMesh->GetName() + "_" + FGuid::NewGuid().ToString(), 
						Flags, 
						PolyMesh->FaceSetNames, 
						LookupMaterialSlot,
						Sample);

					if (StaticMesh)
					{
						ImportedStaticMeshes.Add(StaticMesh);
					}
				}
			}
		}
	};
	

	EFrameReadFlags ReadFlags = ( ImportSettings->StaticMeshSettings.bPropagateMatrixTransformations ? EFrameReadFlags::ApplyMatrix : EFrameReadFlags::None ) | EFrameReadFlags::ForceSingleThreaded;
	AbcFile->ProcessFrames(Func, ReadFlags);
	
	TArray<UObject*> Assets;
	Assets.Append(ImportedStaticMeshes);
	SetMetaData(Assets);

	return ImportedStaticMeshes;
}

UGeometryCache* FAbcImporter::ImportAsGeometryCache(UObject* InParent, EObjectFlags Flags)
{
	// Create a GeometryCache instance
	bool bObjectAlreadyExists = false;
	UGeometryCache* GeometryCache = CreateObjectInstance<UGeometryCache>(InParent, InParent != GetTransientPackage() ? FPaths::GetBaseFilename(InParent->GetName()) : (FPaths::GetBaseFilename(AbcFile->GetFilePath()) + "_" + FGuid::NewGuid().ToString()), Flags, bObjectAlreadyExists);

	// Only import data if a valid object was created
	if (GeometryCache)
	{
		TArray<TUniquePtr<FComponentReregisterContext>> ReregisterContexts;
		for (TObjectIterator<UGeometryCacheComponent> CacheIt; CacheIt; ++CacheIt)
		{
			if (CacheIt->GetGeometryCache() == GeometryCache)
			{	
				ReregisterContexts.Add(MakeUnique<FComponentReregisterContext>(*CacheIt));
			}
		}
		
		// In case this is a reimport operation
		GeometryCache->ClearForReimporting();

		// Load the default material for later usage
		UMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
		check(DefaultMaterial);

		auto CreateMaterial = [this, GeometryCache, DefaultMaterial, InParent, Flags](const FString& FaceSetName)
		{
			FName MaterialName(*FaceSetName);

			UMaterialInterface* Material = AbcImporterUtilities::RetrieveMaterial(*AbcFile, FaceSetName, InParent, Flags);
			check(Material);

			if (Material != DefaultMaterial)
			{
				Material->PostEditChange();
			}

			GeometryCache->Materials.Add(Material);
			GeometryCache->MaterialSlotNames.Add(MaterialName);
		};

		// Add tracks
		const int32 NumPolyMeshes = AbcFile->GetNumPolyMeshes();
		if (NumPolyMeshes != 0)
		{
			TArray<UGeometryCacheTrackStreamable*> Tracks;

			const bool bContainsHeterogeneousMeshes = AbcFile->ContainsHeterogeneousMeshes();
			if (ImportSettings->GeometryCacheSettings.bApplyConstantTopologyOptimizations && bContainsHeterogeneousMeshes)
			{
				TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("HeterogeneousMeshesAndForceSingle", "Unable to enforce constant topology optimizations as the imported tracks contain topology varying data."));
				FAbcImportLogger::AddImportMessage(Message);
			}

			// Use new feedback context to override the one coming from the ReimportManager to be able to show the ScopedSlowTask below
			FFeedbackContextEditor GeoCacheContext;
			const FString AssetName = GeometryCache->GetName();
			const int32 NumFrames = (ImportSettings->SamplingSettings.FrameEnd + 1) - ImportSettings->SamplingSettings.FrameStart;
			int32 TotalFrames = 0;

			FString Title = FString::Format(TEXT("Importing frames for {0} ({1}/{2})"), { AssetName, 0,  NumFrames });
			FScopedSlowTask SlowTask(static_cast<float>(NumFrames), FText::FromString(Title), true, GeoCacheContext);
			SlowTask.MakeDialog(true);
			float CompletedFrames = 0.0f;

			if (ImportSettings->GeometryCacheSettings.bFlattenTracks)
			{
				//UGeometryCacheCodecRaw* Codec = NewObject<UGeometryCacheCodecRaw>(GeometryCache, FName(*FString(TEXT("Flattened_Codec"))), RF_Public);
				UGeometryCacheCodecV1* Codec = NewObject<UGeometryCacheCodecV1>(GeometryCache, FName(*FString(TEXT("Flattened_Codec"))), RF_Public);
				Codec->InitializeEncoder(ImportSettings->GeometryCacheSettings.CompressedPositionPrecision, ImportSettings->GeometryCacheSettings.CompressedTextureCoordinatesNumberOfBits);
				UGeometryCacheTrackStreamable* Track = NewObject<UGeometryCacheTrackStreamable>(GeometryCache, FName(*FString(TEXT("Flattened_Track"))), RF_Public);

				const bool bCalculateMotionVectors = (ImportSettings->GeometryCacheSettings.MotionVectors == EAbcGeometryCacheMotionVectorsImport::CalculateMotionVectorsDuringImport);
				Track->BeginCoding(Codec, ImportSettings->GeometryCacheSettings.bApplyConstantTopologyOptimizations && !bContainsHeterogeneousMeshes, bCalculateMotionVectors, ImportSettings->GeometryCacheSettings.bOptimizeIndexBuffers);
				Tracks.Add(Track);
				
				const TArray<FAbcPolyMesh*>& PolyMeshes = AbcFile->GetPolyMeshes();
				TArray<float> FrameTimes;
				FrameTimes.SetNum(NumFrames);
				
				const int32 NumTracks = Tracks.Num();
				int32 PreviousNumVertices = 0;
				TFunction<void(int32, FAbcFile*)> Callback = [this, &Tracks, &PolyMeshes, &PreviousNumVertices, &FrameTimes, &SlowTask, &CompletedFrames, &TotalFrames, &AssetName, NumFrames](int32 FrameIndex, const FAbcFile* InAbcFile)
				{
					const bool bUseVelocitiesAsMotionVectors = (ImportSettings->GeometryCacheSettings.MotionVectors == EAbcGeometryCacheMotionVectorsImport::ImportAbcVelocitiesAsMotionVectors);
					FGeometryCacheMeshData MeshData;
					bool bConstantTopology = true;
					const bool bStoreImportedVertexNumbers = ImportSettings->GeometryCacheSettings.bStoreImportedVertexNumbers;

					int32 FrameTimeIndex = FMath::Max(0, FrameIndex - ImportSettings->SamplingSettings.FrameStart);
					AbcImporterUtilities::MergePolyMeshesToMeshData(FrameIndex, ImportSettings->SamplingSettings.FrameStart, AbcFile->GetSecondsPerFrame(), bUseVelocitiesAsMotionVectors,
						PolyMeshes,
						AbcFile->GetLookupMaterialSlot(),
						FrameTimes[FrameTimeIndex],
						MeshData, PreviousNumVertices, bConstantTopology, bStoreImportedVertexNumbers);
					
					const float FrameRate = InAbcFile->GetFramerate();

					// Convert frame times to frame numbers and back to time to avoid float imprecision
					const float FrameTime = static_cast<float>(FMath::RoundToInt(FrameTimes[FrameTimeIndex] * FrameRate) - FMath::RoundToInt(InAbcFile->GetImportTimeOffset() * FrameRate)) / FrameRate;
					Tracks[0]->AddMeshSample(MeshData, FrameTime, bConstantTopology);
					
					++CompletedFrames;
					if (IsInGameThread())
					{
						TotalFrames += int32(CompletedFrames);
						FString Title = FString::Format(TEXT("Importing frames for {0} ({1}/{2})"), { AssetName, TotalFrames,  NumFrames });
						SlowTask.EnterProgressFrame(CompletedFrames, FText::FromString(Title));
						CompletedFrames = 0.0f;
					}					
				};

				if (!AbcFile->ProcessFrames(Callback, EFrameReadFlags::ApplyMatrix, &SlowTask))
				{
					UE_LOGF(LogAbcImporter, Warning, "Alembic geometry cache import was interrupted");
				}

				// Now add materials for all the unique face set names
				for (const FString& FaceSetName : AbcFile->GetUniqueFaceSetNames())
				{
					CreateMaterial(FaceSetName);
				}
			}
			else
			{
				uint32 MaterialOffset = 0;
				TArray<int32> MaterialOffsets;
				TArray<FAbcPolyMesh*> ImportPolyMeshes;

				const TArray<FAbcPolyMesh*>& PolyMeshes = AbcFile->GetPolyMeshes();
				for (FAbcPolyMesh* PolyMesh : PolyMeshes)
				{
					if (PolyMesh->bShouldImport)
					{
						FName BaseName = FName(*(PolyMesh->GetName()));
						//UGeometryCacheCodecRaw* Codec = NewObject<UGeometryCacheCodecRaw>(GeometryCache, FName(*(PolyMesh->GetName() + FString(TEXT("_Codec")))), RF_Public);
						FName CodecName = MakeUniqueObjectName(GeometryCache, UGeometryCacheCodecV1::StaticClass(), FName(BaseName.ToString() + FString(TEXT("_Codec"))));
						UGeometryCacheCodecV1* Codec = NewObject<UGeometryCacheCodecV1>(GeometryCache, CodecName, RF_Public);
						Codec->InitializeEncoder(ImportSettings->GeometryCacheSettings.CompressedPositionPrecision, ImportSettings->GeometryCacheSettings.CompressedTextureCoordinatesNumberOfBits);

						FName TrackName = MakeUniqueObjectName(GeometryCache, UGeometryCacheTrackStreamable::StaticClass(), BaseName);
						UGeometryCacheTrackStreamable* Track = NewObject<UGeometryCacheTrackStreamable>(GeometryCache, TrackName, RF_Public);

						const bool bCalculateMotionVectors = (ImportSettings->GeometryCacheSettings.MotionVectors == EAbcGeometryCacheMotionVectorsImport::CalculateMotionVectorsDuringImport);
						Track->BeginCoding(Codec, ImportSettings->GeometryCacheSettings.bApplyConstantTopologyOptimizations && !bContainsHeterogeneousMeshes, bCalculateMotionVectors, ImportSettings->GeometryCacheSettings.bOptimizeIndexBuffers);

						ImportPolyMeshes.Add(PolyMesh);
						Tracks.Add(Track);
						MaterialOffsets.Add(MaterialOffset);

						// Add materials for this Mesh Object
						const uint32 NumMaterials = PolyMesh->FaceSetNames.Num();
						for (uint32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
						{
							CreateMaterial(PolyMesh->FaceSetNames[MaterialIndex]);
						}

						MaterialOffset += NumMaterials;
					}
				}

				const int32 NumTracks = Tracks.Num();
				TFunction<void(int32, FAbcFile*)> Callback = [this, NumTracks, &ImportPolyMeshes, &Tracks, &MaterialOffsets, &SlowTask, &CompletedFrames, &TotalFrames, &AssetName, NumFrames](int32 FrameIndex, const FAbcFile* InAbcFile)
				{
					const float FrameRate = static_cast<float>(InAbcFile->GetFramerate());
					for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
					{
						const FAbcPolyMesh* PolyMesh = ImportPolyMeshes[TrackIndex];
						if (PolyMesh->bShouldImport)
						{
							UGeometryCacheTrackStreamable* Track = Tracks[TrackIndex];

							// Generate the mesh data for this sample
							const bool bVisible = PolyMesh->GetVisibility(FrameIndex);
							// Convert frame times to frame numbers and back to time to avoid float imprecision
							const float FrameTime = static_cast<float>(FMath::RoundToInt(PolyMesh->GetTimeForFrameIndex(FrameIndex) * FrameRate) - FMath::RoundToInt(InAbcFile->GetImportTimeOffset() * FrameRate)) / FrameRate;
							if (bVisible)
							{
								const bool bUseVelocitiesAsMotionVectors = ( ImportSettings->GeometryCacheSettings.MotionVectors == EAbcGeometryCacheMotionVectorsImport::ImportAbcVelocitiesAsMotionVectors );
								const bool bStoreImportedVertexData = ImportSettings->GeometryCacheSettings.bStoreImportedVertexNumbers;
								const FAbcMeshSample* Sample = PolyMesh->GetSample(FrameIndex);
								FGeometryCacheMeshData MeshData;
								AbcImporterUtilities::GeometryCacheDataForMeshSample(MeshData, Sample, MaterialOffsets[TrackIndex], AbcFile->GetSecondsPerFrame(), bUseVelocitiesAsMotionVectors, bStoreImportedVertexData);
								Track->AddMeshSample(MeshData, FrameTime, PolyMesh->bConstantTopology);
							}

							Track->AddVisibilitySample(bVisible, FrameTime);
						}
					}

					++CompletedFrames;
					if (IsInGameThread())
					{
						TotalFrames += int32(CompletedFrames);
						FString Title = FString::Format(TEXT("Importing frames for {0} ({1}/{2})"), { AssetName, TotalFrames,  NumFrames });
						SlowTask.EnterProgressFrame(CompletedFrames, FText::FromString(Title));
						CompletedFrames = 0.0f;
					}					
				};

				if (!AbcFile->ProcessFrames(Callback, EFrameReadFlags::ApplyMatrix, &SlowTask))
				{
					UE_LOGF(LogAbcImporter, Warning, "Alembic geometry cache import was interrupted");
				}
			}

			TArray<FMatrix> Mats;
			Mats.Add(FMatrix::Identity);
			Mats.Add(FMatrix::Identity);

			for (UGeometryCacheTrackStreamable* Track : Tracks)
			{
				TArray<float> MatTimes;
				MatTimes.Add(0.0f);
				MatTimes.Add(AbcFile->GetImportLength() + AbcFile->GetImportTimeOffset());
				Track->SetMatrixSamples(Mats, MatTimes);

				// Some tracks might not have any mesh samples because they are invisible (can happen with bFlattenTracks=false), so skip them
				if (Track->EndCoding())
				{
					GeometryCache->AddTrack(Track);
				}
			}
		}

		// For alembic, for now, we define the duration of the tracks as the duration of the longer track in the whole file so all tracks loop in union
		float MaxDuration = 0.0f;
		for (auto Track : GeometryCache->Tracks)
		{
			MaxDuration = FMath::Max(MaxDuration, Track->GetDuration());
		}
		for (auto Track : GeometryCache->Tracks)
		{
			Track->SetDuration(MaxDuration);
		}
		// Also store the number of frames in the cache
		GeometryCache->SetFrameStartEnd(ImportSettings->SamplingSettings.FrameStart, ImportSettings->SamplingSettings.FrameEnd);
		
		// Update all geometry cache components, TODO move render-data from component to GeometryCache and allow for DDC population
		for (TObjectIterator<UGeometryCacheComponent> CacheIt; CacheIt; ++CacheIt)
		{
			CacheIt->OnObjectReimported(GeometryCache);
		}

		SetMetaData({GeometryCache});
	}
	
	return GeometryCache;
}

TArray<UObject*> FAbcImporter::ImportAsSkeletalMesh(UObject* InParent, EObjectFlags Flags)
{
	// First compress the animation data
	const bool bRunComparison = false;
	const bool bCompressionResult = CompressAnimationDataUsingPCA(ImportSettings->CompressionSettings, bRunComparison);

	TArray<UObject*> GeneratedObjects;

	if (!bCompressionResult)
	{
		return GeneratedObjects;
	}

	// Create a Skeletal mesh instance 
	
	const FString& ObjectName = InParent != GetTransientPackage() ? FPaths::GetBaseFilename(InParent->GetName()) : (FPaths::GetBaseFilename(AbcFile->GetFilePath()) + "_" + FGuid::NewGuid().ToString());
	const FString SanitizedObjectName = ObjectTools::SanitizeObjectName(ObjectName);

	USkeletalMesh* ExistingSkeletalMesh = FindObject<USkeletalMesh>(InParent, *SanitizedObjectName);	
	FSkinnedMeshComponentRecreateRenderStateContext* RecreateExistingRenderStateContext = ExistingSkeletalMesh ? new FSkinnedMeshComponentRecreateRenderStateContext(ExistingSkeletalMesh, false) : nullptr;
	
	bool bMeshAlreadyExists = false;
	USkeletalMesh* SkeletalMesh = CreateObjectInstance<USkeletalMesh>(InParent, ObjectName, Flags, bMeshAlreadyExists);

	// Only import data if a valid object was created
	if (SkeletalMesh)
	{
		// Touch pre edit change
		SkeletalMesh->PreEditChange(nullptr);

		// Retrieve the imported resource structure and allocate a new LOD model
		FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();
		ImportedModel->LODModels.Empty();
		ImportedModel->EmptyOriginalReductionSourceMeshData();
		ImportedModel->LODModels.Add(new FSkeletalMeshLODModel());
		SkeletalMesh->SetNumSourceModels(0);

		FSkeletalMeshLODInfo& NewLODInfo = SkeletalMesh->AddLODInfo();
		NewLODInfo.ReductionSettings.NumOfTrianglesPercentage = 1.0f;
		NewLODInfo.ReductionSettings.NumOfVertPercentage = 1.0f;
		NewLODInfo.ReductionSettings.MaxDeviationPercentage = 0.0f;
		FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[0];

		const FMeshBoneInfo BoneInfo(FName(TEXT("RootBone"), FNAME_Add), TEXT("RootBone_Export"), INDEX_NONE);
		const FTransform BoneTransform;
		{
			FReferenceSkeletonModifier RefSkelModifier(SkeletalMesh->GetRefSkeleton(), SkeletalMesh->GetSkeleton());
			if (RefSkelModifier.FindBoneIndex(BoneInfo.Name) == INDEX_NONE)
			{
				RefSkelModifier.Add(BoneInfo, BoneTransform);
			}
		}


		/* Bounding box according to animation */
		SkeletalMesh->SetImportedBounds(AbcFile->GetArchiveBounds().GetBox());

		bool bBuildSuccess = false;
		TArray<int32> MorphTargetVertexRemapping;
		TArray<int32> UsedVertexIndicesForMorphs;

		{
			FAbcMeshSample MergedMeshSample; // Temporary mesh
			for (const FCompressedAbcData& Data : CompressedMeshData)
			{
				AbcImporterUtilities::AppendMeshSample(&MergedMeshSample, Data.AverageSample);
			}

			MergedMeshSample.TangentX.Empty();
			MergedMeshSample.TangentY.Empty();

			// Forced to 1
			LODModel.NumTexCoords = MergedMeshSample.NumUVSets;
			SkeletalMesh->SetHasVertexColors(true);
			SkeletalMesh->SetVertexColorGuid(FGuid::NewGuid());

			bBuildSuccess = BuildSkeletalMesh(LODModel, SkeletalMesh->GetRefSkeleton(), &MergedMeshSample,
				AbcFile->GetNumMaterialSlots(),
				AbcFile->GetLookupMaterialSlot(),
				MorphTargetVertexRemapping, UsedVertexIndicesForMorphs);
		}

		if (!bBuildSuccess)
		{
			SkeletalMesh->MarkAsGarbage();
			return GeneratedObjects;
		}

		// Create the skeleton object
		FString SkeletonName = FString::Printf(TEXT("%s_Skeleton"), *SkeletalMesh->GetName());
		bool bSkeletonAlreadyExists = false;
		USkeleton* Skeleton = CreateObjectInstance<USkeleton>(InParent, SkeletonName, Flags, bSkeletonAlreadyExists);

		// Merge bones to the selected skeleton
		ensure(Skeleton->MergeAllBonesToBoneTree(SkeletalMesh));
		Skeleton->MarkPackageDirty();
		if (SkeletalMesh->GetSkeleton() != Skeleton)
		{
			SkeletalMesh->SetSkeleton(Skeleton);
			SkeletalMesh->MarkPackageDirty();
		}

		// Create animation sequence for the skeleton
		bool bSequenceAlreadyExists = false;
		UAnimSequence* Sequence = CreateObjectInstance<UAnimSequence>(InParent, FString::Printf(TEXT("%s_Animation"), *SkeletalMesh->GetName()), Flags, bSequenceAlreadyExists);
		Sequence->SetSkeleton(Skeleton);

		int32 ObjectIndex = 0;
		uint32 TriangleOffset = 0;
		uint32 WedgeOffset = 0;
		uint32 VertexOffset = 0;

		IAnimationDataController& Controller = Sequence->GetController();

		const bool bShouldTransact = bSequenceAlreadyExists;
		Controller.OpenBracket(LOCTEXT("ImportAsSkeletalMesh", "Importing Alembic Animation"), bShouldTransact);
		Controller.InitializeModel();

		IAnimationDataModel::FReimportScope ReimportScope(Sequence->GetDataModel());

		const FFrameRate FrameRate(FMath::RoundToInt(AbcFile->GetFramerate()), 1);
		Controller.SetFrameRate(FrameRate, bShouldTransact);	
		const FFrameNumber FrameNumber = FrameRate.AsFrameNumber(AbcFile->GetImportLength());
		Controller.SetNumberOfFrames(FrameNumber, bShouldTransact);

		Sequence->ImportFileFramerate = static_cast<float>(FrameRate.AsDecimal());
		Sequence->ImportResampleFramerate = static_cast<int32>(FrameRate.AsInterval());

		{
			// When ScopedPostEditChange goes out of scope, it will call SkeletalMesh->PostEditChange()
			// while preventing any call to that within the scope
			FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);

			for (FCompressedAbcData& CompressedData : CompressedMeshData)
			{
				FAbcMeshSample* AverageSample = CompressedData.AverageSample;

				if (CompressedData.BaseSamples.Num() > 0)
				{
					const int32 NumBases = CompressedData.BaseSamples.Num();
					int32 NumUsedBases = 0;

					const int32 NumIndices = CompressedData.AverageSample->Indices.Num();

					for (int32 BaseIndex = 0; BaseIndex < NumBases; ++BaseIndex)
					{
						FAbcMeshSample* BaseSample = CompressedData.BaseSamples[BaseIndex];

						// Create new morph target with name based on object and base index
						UMorphTarget* MorphTarget = NewObject<UMorphTarget>(SkeletalMesh, FName(*FString::Printf(TEXT("Base_%i_%i"), BaseIndex, ObjectIndex)));

						// Setup morph target vertices directly
						TArray<FMorphTargetDelta> MorphDeltas;
						GenerateMorphTargetVertices(BaseSample, MorphDeltas, AverageSample, WedgeOffset, MorphTargetVertexRemapping, UsedVertexIndicesForMorphs, VertexOffset, WedgeOffset);

						const bool bCompareNormals = true;
						MorphTarget->PopulateDeltas(MorphDeltas, 0, LODModel.Sections, bCompareNormals);

						const float PercentageOfVerticesInfluences = ((float)MorphTarget->GetMorphLODModels()[0].Vertices.Num() / (float)NumIndices) * 100.0f;
						if (PercentageOfVerticesInfluences > ImportSettings->CompressionSettings.MinimumNumberOfVertexInfluencePercentage)
						{
							SkeletalMesh->RegisterMorphTarget(MorphTarget);
							MorphTarget->MarkPackageDirty();

							// Set up curves
							const TArray<float>& CurveValues = CompressedData.CurveValues[BaseIndex];
							const TArray<float>& TimeValues = CompressedData.TimeValues[BaseIndex];
							// Morph target stuffies
							FString CurveName = FString::Printf(TEXT("Base_%i_%i"), BaseIndex, ObjectIndex);
							FName ConstCurveName = *CurveName;

							// Sets up the morph target curves with the sample values and time keys
							SetupMorphTargetCurves(Skeleton, ConstCurveName, Sequence, CurveValues, TimeValues, Controller, bShouldTransact);
						}
						else
						{
							MorphTarget->MarkAsGarbage();
						}
					}
				}

				WedgeOffset += CompressedData.AverageSample->Indices.Num();
				VertexOffset += CompressedData.AverageSample->Vertices.Num();

				++ObjectIndex;
			}

			// Add a track for translating the RootBone by the samples centers
			// Each mesh has the same samples centers so use the first one
			if (SamplesOffsets.IsSet() && CompressedMeshData.Num() > 0 && CompressedMeshData[0].CurveValues.Num() > 0)
			{
				const int32 NumSamples = CompressedMeshData[0].CurveValues[0].Num(); // We might have less bases than we have samples, so use the number of curve values here

				FRawAnimSequenceTrack RootBoneTrack;
				RootBoneTrack.PosKeys.Reserve(NumSamples);
				RootBoneTrack.RotKeys.Reserve(NumSamples);
				RootBoneTrack.ScaleKeys.Reserve(NumSamples);

				for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
				{
					const FVector SampleOffset = SamplesOffsets.GetValue()[SampleIndex];
					RootBoneTrack.PosKeys.Add((FVector3f)SampleOffset);

					RootBoneTrack.RotKeys.Add(FQuat4f::Identity);
					RootBoneTrack.ScaleKeys.Add(FVector3f::OneVector);
				}

				const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
				const TArray<FMeshBoneInfo>& BonesInfo = RefSkeleton.GetRawRefBoneInfo();
				
				Controller.AddBoneCurve(BonesInfo[0].Name, bShouldTransact);
				Controller.SetBoneTrackKeys(BonesInfo[0].Name, RootBoneTrack.PosKeys, RootBoneTrack.RotKeys, RootBoneTrack.ScaleKeys, bShouldTransact);
			}

			// Set recompute tangent flag on skeletal mesh sections
			for (FSkelMeshSection& Section : LODModel.Sections)
			{
				Section.bRecomputeTangent = true;
			}

			SkeletalMesh->CalculateInvRefMatrices();
		}

		UMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
		check(DefaultMaterial);

		// Build the material slots : one for each faceset
		for (const FString& FaceSetName : AbcFile->GetUniqueFaceSetNames())
		{
			UMaterialInterface* Material = AbcImporterUtilities::RetrieveMaterial(*AbcFile, FaceSetName, InParent, Flags);
			if (Material == nullptr)
			{
				Material = DefaultMaterial;
			}
			else if (Material != DefaultMaterial)
			{
				Material->PostEditChange();
			}

			FName MaterialName(*FaceSetName);
			SkeletalMesh->GetMaterials().Add(FSkeletalMaterial(Material, true, false, MaterialName, MaterialName));
		}

		SkeletalMesh->MarkPackageDirty();

		Controller.NotifyPopulated();

		Controller.CloseBracket(bShouldTransact);

		Sequence->PostEditChange();
		Sequence->SetPreviewMesh(SkeletalMesh);
		Sequence->MarkPackageDirty();

		Skeleton->SetPreviewMesh(SkeletalMesh);
		Skeleton->PostEditChange();

		GeneratedObjects.Add(SkeletalMesh);
		GeneratedObjects.Add(Skeleton);
		GeneratedObjects.Add(Sequence);

		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		AssetEditorSubsystem->CloseAllEditorsForAsset(Skeleton);
		AssetEditorSubsystem->CloseAllEditorsForAsset(SkeletalMesh);
		AssetEditorSubsystem->CloseAllEditorsForAsset(Sequence);
	}

	if (RecreateExistingRenderStateContext)
	{
		delete RecreateExistingRenderStateContext;
	}

	SetMetaData(GeneratedObjects);

	return GeneratedObjects;
}

void FAbcImporter::SetupMorphTargetCurves(USkeleton* Skeleton, FName ConstCurveName, UAnimSequence* Sequence, const TArray<float> &CurveValues, const TArray<float>& TimeValues, IAnimationDataController& Controller, bool bShouldTransact)
{
	// Need curve metadata for the AnimSequence to playback. Can be either on the Skeleton or SKelMesh,
	// but by default for FBX import it's on the Skeleton so do the same for Alembic
	const bool bMaterialCurve = false;
	const bool bMorphTargetCurve = true;
	Skeleton->AccumulateCurveMetaData(ConstCurveName, bMaterialCurve, bMorphTargetCurve);

	FAnimationCurveIdentifier CurveId(ConstCurveName, ERawCurveTrackTypes::RCT_Float);
	Controller.AddCurve(CurveId, EAnimAssetCurveFlags::AACF_Editable, bShouldTransact);

	const FFloatCurve* NewCurve = Sequence->GetDataModel()->FindFloatCurve(CurveId);
	ensure(NewCurve);

	FRichCurve RichCurve;
	for (int32 KeyIndex = 0; KeyIndex < CurveValues.Num(); ++KeyIndex)
	{
		const float CurveValue = CurveValues[KeyIndex];
		const float TimeValue = TimeValues[KeyIndex];

		FKeyHandle NewKeyHandle = RichCurve.AddKey(TimeValue, CurveValue, false);

		ERichCurveInterpMode NewInterpMode = RCIM_Linear;
		ERichCurveTangentMode NewTangentMode = RCTM_Auto;
		ERichCurveTangentWeightMode NewTangentWeightMode = RCTWM_WeightedNone;

		RichCurve.SetKeyInterpMode(NewKeyHandle, NewInterpMode);
		RichCurve.SetKeyTangentMode(NewKeyHandle, NewTangentMode);
		RichCurve.SetKeyTangentWeightMode(NewKeyHandle, NewTangentWeightMode);
	}

	Controller.SetCurveKeys(CurveId, RichCurve.GetConstRefOfKeys(), bShouldTransact);
}

void FAbcImporter::SetMetaData(const TArray<UObject*>& Objects)
{
	const TArray<FAbcFile::FMetaData> ArchiveMetaData = AbcFile->GetArchiveMetaData();
	for (UObject* Object : Objects)
	{
		if (Object)
		{
			for (const FAbcFile::FMetaData& MetaData : ArchiveMetaData)
			{
				Object->GetPackage()->GetMetaData().SetValue(Object, *MetaData.Key, *MetaData.Value);
			}
		}
	}
}

const bool FAbcImporter::CompressAnimationDataUsingPCA(const FAbcCompressionSettings& InCompressionSettings, const bool bRunComparison /*= false*/)
{
	const TArray<FAbcPolyMesh*>& PolyMeshes = AbcFile->GetPolyMeshes();
	
	// Split up poly mesh objects into constant and animated objects to process
	TArray<FAbcPolyMesh*> PolyMeshesToCompress;
	TArray<FAbcPolyMesh*> ConstantPolyMeshObjects;
	for (FAbcPolyMesh* PolyMesh : PolyMeshes)
	{
		if (PolyMesh->bShouldImport && PolyMesh->bConstantTopology)
		{
			if (PolyMesh->IsConstant() && PolyMesh->bConstantTransformation)
			{
				ConstantPolyMeshObjects.Add(PolyMesh);
			}
			else if (!PolyMesh->IsConstant() || (InCompressionSettings.bBakeMatrixAnimation && !PolyMesh->bConstantTransformation))
			{
				PolyMeshesToCompress.Add(PolyMesh);
			}
		}
	}

	const bool bEnableSamplesOffsets = (ConstantPolyMeshObjects.Num() == 0); // We can't offset constant meshes since they don't have morph targets

	bool bResult = true;
	const int32 NumPolyMeshesToCompress = PolyMeshesToCompress.Num();
	if (NumPolyMeshesToCompress)
	{
		if (InCompressionSettings.bMergeMeshes)
		{
			// Merged path
			TArray<FVector3f> AverageVertexData;
			TArray<FVector3f> AverageNormalData;

			float MinTime = FLT_MAX;
			float MaxTime = -FLT_MAX;
			int32 NumSamples = 0;

			TUniquePtr<FScopedSlowTask> SlowTask = MakeUnique<FScopedSlowTask>(static_cast<float>((ImportSettings->SamplingSettings.FrameEnd + 1) - ImportSettings->SamplingSettings.FrameStart + 1), FText::FromString(FString(TEXT("Merging meshes"))));
			SlowTask->MakeDialog(true);

			TArray<uint32> ObjectVertexOffsets;
			TArray<uint32> ObjectIndexOffsets;
			float CompletedFrames = 0.0f;
			TFunction<void(int32, FAbcFile*)> MergedMeshesFunc =
				[this, PolyMeshesToCompress, &MinTime, &MaxTime, &NumSamples, &ObjectVertexOffsets, &ObjectIndexOffsets, &AverageVertexData, &AverageNormalData, NumPolyMeshesToCompress, &SlowTask, &CompletedFrames]
				(int32 FrameIndex, FAbcFile* InFile)
				{
					const float FrameRate = static_cast<float>(AbcFile->GetFramerate());
					for (int32 MeshIndex = 0; MeshIndex < NumPolyMeshesToCompress; ++MeshIndex)
					{
						FAbcPolyMesh* PolyMesh = PolyMeshesToCompress[MeshIndex];

						// Convert frame times to frame numbers and back to time to avoid float imprecision
						const float FrameTime = static_cast<float>(FMath::RoundToInt(PolyMesh->GetTimeForFrameIndex(FrameIndex) * FrameRate) - FMath::RoundToInt(AbcFile->GetImportTimeOffset() * FrameRate)) / FrameRate;
						MinTime = FMath::Min(MinTime, FrameTime);
						MaxTime = FMath::Max(MaxTime, FrameTime);

						if (ObjectVertexOffsets.Num() != NumPolyMeshesToCompress)
						{
							ObjectVertexOffsets.Add(AverageVertexData.Num());
							ObjectIndexOffsets.Add(AverageNormalData.Num());
							AverageVertexData.Append(PolyMesh->GetSample(FrameIndex)->Vertices);
							AverageNormalData.Append(PolyMesh->GetSample(FrameIndex)->Normals);
						}
						else
						{
							for (int32 VertexIndex = 0; VertexIndex < PolyMesh->GetSample(FrameIndex)->Vertices.Num(); ++VertexIndex)
							{
								AverageVertexData[VertexIndex + ObjectVertexOffsets[MeshIndex]] += PolyMesh->GetSample(FrameIndex)->Vertices[VertexIndex];
							}

							for (int32 Index = 0; Index < PolyMesh->GetSample(FrameIndex)->Indices.Num(); ++Index)
							{
								AverageNormalData[Index + ObjectIndexOffsets[MeshIndex]] += PolyMesh->GetSample(FrameIndex)->Normals[Index];
							}
						}
					}

					++NumSamples;
					++CompletedFrames;

					if (IsInGameThread())
					{
						SlowTask->EnterProgressFrame(CompletedFrames);
						CompletedFrames = 0.0f;
					}					
				};

			EFrameReadFlags Flags = EFrameReadFlags::PositionAndNormalOnly;
			if (ImportSettings->CompressionSettings.bBakeMatrixAnimation)
			{
				Flags |= EFrameReadFlags::ApplyMatrix;
			}

			{
				// Check the first frame to see if the Alembic can be imported as a skeletal mesh due to memory constraints
				AbcFile->ReadFrame(AbcFile->GetStartFrameIndex(), Flags, 0);
				MergedMeshesFunc(AbcFile->GetStartFrameIndex(), AbcFile);
				AbcFile->CleanupFrameData(0);

				const int32 NumFrames = AbcFile->GetEndFrameIndex() - AbcFile->GetStartFrameIndex() + 1;
				const uint64 NumMatrixElements = uint64(AverageVertexData.Num()) * 3 * NumFrames;
				bool bTriggerWarning = false;
				int32 NumElementsWarning = 0;
				uint64 NumMatrixElementsWarning = 0;
				if (!IntFitsIn<int32>(NumMatrixElements))
				{
					NumElementsWarning = AverageVertexData.Num();
					NumMatrixElementsWarning = NumMatrixElements;
					bTriggerWarning = true;
				}

				const uint64 NumNormalsMatrixElements = uint64(AverageNormalData.Num()) * 3 * NumFrames;
				if (!IntFitsIn<int32>(NumNormalsMatrixElements))
				{
					if (AverageNormalData.Num() > NumElementsWarning || NumNormalsMatrixElements > NumMatrixElementsWarning)
					{
						NumElementsWarning = AverageNormalData.Num();
						NumMatrixElementsWarning = NumNormalsMatrixElements;
					}
					bTriggerWarning = true;
				}

				if (bTriggerWarning)
				{
					UE_LOGF(LogAbcImporter, Warning, "Vertex matrix has %llu elements because the mesh has %d vertices and the animation has %d frames. This can cause the import to take a long time and use a lot of memory.",
						NumMatrixElementsWarning, NumElementsWarning, NumFrames);

					const FText Title = LOCTEXT("AbcSkelMeshImportWarningTitle", "Proceed with import?");
					const FText Message = LOCTEXT("AbcSkelMeshImportWarningMessage", "Warning: Due to the mesh size and animation length, the import may take a long time and may run out of memory and crash. Do you want to continue?\n"
						"If not, you may try reducing the animation import range (will use less memory) or use No Compression as the Base Calculation Type (faster, but will use more memory) or import as Geometry Cache.");

					const EAppReturnType::Type DialogResponse = FMessageDialog::Open(EAppMsgType::OkCancel, EAppReturnType::Ok, Message, Title);
					if (DialogResponse != EAppReturnType::Ok)
					{
					return false;
				}
				}

				AverageVertexData.Reset();
				AverageNormalData.Reset();
				ObjectVertexOffsets.Reset();
				ObjectIndexOffsets.Reset();

				MinTime = FLT_MAX;
				MaxTime = -FLT_MAX;
				NumSamples = 0;
			}

			if (!AbcFile->ProcessFrames(MergedMeshesFunc, Flags, SlowTask.Get()))
			{
				UE_LOGF(LogAbcImporter, Warning, "Alembic skeletal mesh import was interrupted");
				return false;
			}
			SlowTask.Reset();

			// Average out vertex data
			FBox AverageBoundingBox(ForceInit);
			const float Multiplier = 1.0f / static_cast<float>(FMath::Max(NumSamples, 1.f));
			for (FVector3f& Vertex : AverageVertexData)
			{
				Vertex *= Multiplier;
				AverageBoundingBox += (FVector)Vertex;
			}
			const FVector AverageSampleCenter = AverageBoundingBox.GetCenter();

			for (FVector3f& Normal : AverageNormalData)
			{
				Normal = Normal.GetSafeNormal();
			}

			// Allocate compressed mesh data object
			CompressedMeshData.AddDefaulted();
			FCompressedAbcData& CompressedData = CompressedMeshData.Last();

			FAbcMeshSample MergedZeroFrameSample;
			for (FAbcPolyMesh* PolyMesh : PolyMeshesToCompress)
			{
				AbcImporterUtilities::AppendMeshSample(&MergedZeroFrameSample, PolyMesh->GetTransformedFirstSample());
			}

			const uint32 NumVertices = AverageVertexData.Num();
			const uint32 NumMatrixRows = NumVertices * 3;
			const uint32 NumIndices = AverageNormalData.Num();
			const uint32 NumNormalsMatrixRows = NumIndices * 3;

			TArray64<float> OriginalMatrix;
			OriginalMatrix.AddZeroed(NumMatrixRows * NumSamples);

			TArray64<float> OriginalNormalsMatrix;
			OriginalNormalsMatrix.AddZeroed(NumNormalsMatrixRows * NumSamples);

			if (bEnableSamplesOffsets)
			{
				SamplesOffsets.Emplace();
				SamplesOffsets.GetValue().AddZeroed(NumSamples);
			}

			SlowTask = MakeUnique<FScopedSlowTask>(static_cast<float>((ImportSettings->SamplingSettings.FrameEnd + 1) - ImportSettings->SamplingSettings.FrameStart), FText::FromString(FString(TEXT("Generating matrices"))));
			SlowTask->MakeDialog(true);

			CompletedFrames = 0.0f;

			uint32 GenerateMatrixSampleIndex = 0;
			TFunction<void(int32, FAbcFile*)> GenerateMatrixFunc =
				[this, PolyMeshesToCompress, NumPolyMeshesToCompress, &OriginalMatrix, &OriginalNormalsMatrix, &AverageVertexData, &AverageNormalData, &NumSamples,
					&ObjectVertexOffsets, &ObjectIndexOffsets, &GenerateMatrixSampleIndex, &AverageSampleCenter, &SlowTask, &CompletedFrames]
				(int32 FrameIndex, FAbcFile* InFile)
				{
					FVector SampleOffset = FVector::ZeroVector;
					if (SamplesOffsets.IsSet())
					{
						FBox BoundingBox(ForceInit);

						// For each object generate the delta frame data for the PCA compression
						for (FAbcPolyMesh* PolyMesh : PolyMeshesToCompress)
						{
							const TArray<FVector3f>& Vertices = PolyMesh->GetSample(FrameIndex)->Vertices;

							for ( const FVector3f& Vertex : Vertices )
							{
								BoundingBox += (FVector)Vertex;
							}
						}

						SampleOffset = BoundingBox.GetCenter() - AverageSampleCenter;
						SamplesOffsets.GetValue()[GenerateMatrixSampleIndex] = SampleOffset;
					}

					// For each object generate the delta frame data for the PCA compression
					for (int32 MeshIndex = 0; MeshIndex < NumPolyMeshesToCompress; ++MeshIndex)
					{
						FAbcPolyMesh* PolyMesh = PolyMeshesToCompress[MeshIndex];
						const TArray<FVector3f>& Vertices = PolyMesh->GetSample(FrameIndex)->Vertices;
						const TArray<FVector3f>& Normals = PolyMesh->GetSample(FrameIndex)->Normals;

						AbcImporterUtilities::GenerateDeltaFrameDataMatrix(Vertices, Normals, AverageVertexData, AverageNormalData, GenerateMatrixSampleIndex,
							ObjectVertexOffsets[MeshIndex], ObjectIndexOffsets[MeshIndex], SampleOffset, OriginalMatrix, OriginalNormalsMatrix);
					}

					++GenerateMatrixSampleIndex;
					++CompletedFrames;

					if (IsInGameThread())
					{
						SlowTask->EnterProgressFrame(CompletedFrames);
						CompletedFrames = 0.0f;
					}					
				};

			if (!AbcFile->ProcessFrames(GenerateMatrixFunc, Flags, SlowTask.Get()))
			{
				UE_LOGF(LogAbcImporter, Warning, "Alembic skeletal mesh import was interrupted");
				return false;
			}
			SlowTask.Reset();

			// Perform compression
			TArray64<float> OutU, OutV, OutNormalsU;
			TArrayView64<float> BasesMatrix;
			TArrayView64<float> NormalsBasesMatrix;
			uint32 NumUsedSingularValues = NumSamples;

			if (InCompressionSettings.BaseCalculationType != EBaseCalculationType::NoCompression)
			{
				const float PercentageOfTotalBases = (InCompressionSettings.BaseCalculationType == EBaseCalculationType::PercentageBased ? InCompressionSettings.PercentageOfTotalBases / 100.0f : 1.0f);
				const int32 NumberOfBases = InCompressionSettings.BaseCalculationType == EBaseCalculationType::FixedNumber ? InCompressionSettings.MaxNumberOfBases : 0;

				NumUsedSingularValues = PerformSVDCompression(OriginalMatrix, OriginalNormalsMatrix, NumSamples, PercentageOfTotalBases, NumberOfBases, OutU, OutNormalsU, OutV);
				BasesMatrix = OutU;
				NormalsBasesMatrix= OutNormalsU;
			}
			else
			{
				OutV.AddDefaulted(NumSamples * NumSamples);

				for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
				{
					for (int32 CurveIndex = 0; CurveIndex < NumSamples; ++CurveIndex)
					{
						float Weight = 0.f;

						if ( SampleIndex == CurveIndex )
						{
							Weight = 1.f;
						}

						OutV[SampleIndex + (NumSamples * CurveIndex )] = Weight;
					}
				}

				BasesMatrix = OriginalMatrix;
				NormalsBasesMatrix = OriginalNormalsMatrix;
			}

			// Set up average frame 
			CompressedData.AverageSample = new FAbcMeshSample(MergedZeroFrameSample);
			FMemory::Memcpy(CompressedData.AverageSample->Vertices.GetData(), AverageVertexData.GetData(), AverageVertexData.GetTypeSize() * NumVertices);
			FMemory::Memcpy(CompressedData.AverageSample->Normals.GetData(), AverageNormalData.GetData(), AverageNormalData.GetTypeSize() * NumIndices);

			const float FrameStep = (MaxTime - MinTime) / (float)(NumSamples - 1);
			AbcImporterUtilities::GenerateCompressedMeshData(CompressedData, NumUsedSingularValues, NumSamples, BasesMatrix, NormalsBasesMatrix, OutV, FrameStep, FMath::Max(MinTime, 0.0f));

			if (bRunComparison)
			{
				CompareCompressionResult(OriginalMatrix, NumSamples, NumUsedSingularValues, BasesMatrix, OutV, THRESH_POINTS_ARE_SAME);
				CompareCompressionResult(OriginalNormalsMatrix, NumSamples, NumUsedSingularValues, NormalsBasesMatrix, OutV, THRESH_NORMALS_ARE_SAME);
			}
		}
		else
		{
			TArray<float> MinTimes;
			TArray<float> MaxTimes;
			TArray<TArray<FVector3f>> AverageVertexData;
			TArray<TArray<FVector3f>> AverageNormalData;

			MinTimes.AddZeroed(NumPolyMeshesToCompress);
			MaxTimes.AddZeroed(NumPolyMeshesToCompress);
			AverageVertexData.AddDefaulted(NumPolyMeshesToCompress);
			AverageNormalData.AddDefaulted(NumPolyMeshesToCompress);
			
			TUniquePtr<FScopedSlowTask> SlowTask = MakeUnique<FScopedSlowTask>(static_cast<float>((ImportSettings->SamplingSettings.FrameEnd + 1) - ImportSettings->SamplingSettings.FrameStart + 1), FText::FromString(FString(TEXT("Processing meshes"))));
			SlowTask->MakeDialog(true);
			
			int32 NumSamples = 0;
			float CompletedFrames = 0.0f;
			TFunction<void(int32, FAbcFile*)> IndividualMeshesFunc =
				[this, NumPolyMeshesToCompress, &PolyMeshesToCompress, &MinTimes, &MaxTimes, &NumSamples, &AverageVertexData, &AverageNormalData, &SlowTask, &CompletedFrames]
				(int32 FrameIndex, FAbcFile* InFile)
			{
				const float FrameRate = static_cast<float>(AbcFile->GetFramerate());
				// Each individual object creates a compressed data object
				for (int32 MeshIndex = 0; MeshIndex < NumPolyMeshesToCompress; ++MeshIndex)
				{
					FAbcPolyMesh* PolyMesh = PolyMeshesToCompress[MeshIndex];
					TArray<FVector3f>& AverageVertices = AverageVertexData[MeshIndex];
					TArray<FVector3f>& AverageNormals = AverageNormalData[MeshIndex];

					if (AverageVertices.Num() == 0)
					{
						MinTimes[MeshIndex] = FLT_MAX;
						MaxTimes[MeshIndex] = -FLT_MAX;
						AverageVertices.Append(PolyMesh->GetSample(FrameIndex)->Vertices);
						AverageNormals.Append(PolyMesh->GetSample(FrameIndex)->Normals);
					}
					else
					{
						const TArray<FVector3f>& CurrentVertices = PolyMesh->GetSample(FrameIndex)->Vertices;
						for (int32 VertexIndex = 0; VertexIndex < AverageVertices.Num(); ++VertexIndex)
						{
							AverageVertices[VertexIndex] += CurrentVertices[VertexIndex];
						}

						for (int32 Index = 0; Index < PolyMesh->GetSample(FrameIndex)->Indices.Num(); ++Index)
						{
							AverageNormals[Index] += PolyMesh->GetSample(FrameIndex)->Normals[Index];
						}
					}

					const float FrameTime = static_cast<float>(FMath::RoundToInt(PolyMesh->GetTimeForFrameIndex(FrameIndex) * FrameRate) - FMath::RoundToInt(AbcFile->GetImportTimeOffset() * FrameRate)) / FrameRate;
					MinTimes[MeshIndex] = FMath::Min(MinTimes[MeshIndex], FrameTime);
					MaxTimes[MeshIndex] = FMath::Max(MaxTimes[MeshIndex], FrameTime);
				}

				for (int32 MeshIndex = 0; MeshIndex < NumPolyMeshesToCompress; ++MeshIndex)
				{
					TArray<FVector3f>& AverageNormals = AverageNormalData[MeshIndex];

					for (FVector3f& AverageNormal : AverageNormals)
					{
						AverageNormal = AverageNormal.GetSafeNormal();
					}
				}

				++NumSamples;
				++CompletedFrames;

				if (IsInGameThread())
				{
					SlowTask->EnterProgressFrame(CompletedFrames);
					CompletedFrames = 0.0f;
				}					
			};

			EFrameReadFlags Flags = EFrameReadFlags::PositionAndNormalOnly;
			if (ImportSettings->CompressionSettings.bBakeMatrixAnimation)
			{
				Flags |= EFrameReadFlags::ApplyMatrix;
			}

			{
				// Check the first frame to see if the Alembic can be imported as a skeletal mesh due to memory constraints
				AbcFile->ReadFrame(AbcFile->GetStartFrameIndex(), Flags, 0);
				IndividualMeshesFunc(AbcFile->GetStartFrameIndex(), AbcFile);
				AbcFile->CleanupFrameData(0);

				const int32 NumFrames = AbcFile->GetEndFrameIndex() - AbcFile->GetStartFrameIndex() + 1;
				bool bTriggerWarning = false;
				int32 NumElementsWarning = 0;
				uint64 NumMatrixElementsWarning = 0;
				for (int32 MeshIndex = 0; MeshIndex < NumPolyMeshesToCompress; ++MeshIndex)
				{
					const uint64 NumMatrixElements = uint64(AverageVertexData[MeshIndex].Num()) * 3 * NumFrames;
					if (!IntFitsIn<int32>(NumMatrixElements))
					{
						NumElementsWarning = AverageVertexData[MeshIndex].Num();
						NumMatrixElementsWarning = NumMatrixElements;
						bTriggerWarning = true;
						break;
					}

					const uint64 NumNormalsMatrixElements = uint64(AverageNormalData[MeshIndex].Num()) * 3 * NumFrames;
					if (!IntFitsIn<int32>(NumNormalsMatrixElements))
					{
						NumElementsWarning = AverageNormalData[MeshIndex].Num();
						NumMatrixElementsWarning = NumNormalsMatrixElements;
						bTriggerWarning = true;
						break;
					}
				}

				if (bTriggerWarning)
				{
					UE_LOGF(LogAbcImporter, Warning, "Vertex matrix has %llu elements because the mesh has %d vertices and the animation has %d frames. This can cause the import to take a long time and use a lot of memory.",
						NumMatrixElementsWarning, NumElementsWarning, NumFrames);

					const FText Title = LOCTEXT("AbcSkelMeshImportWarningTitle", "Proceed with import?");
					const FText Message = LOCTEXT("AbcSkelMeshImportWarningMessage", "Warning: Due to the mesh size and animation length, the import may take a long time and may run out of memory and crash. Do you want to continue?\n"
						"If not, you may try reducing the animation import range (will use less memory) or use No Compression as the Base Calculation Type (faster, but will use more memory) or import as Geometry Cache.");

					const EAppReturnType::Type DialogResponse = FMessageDialog::Open(EAppMsgType::OkCancel, EAppReturnType::Ok, Message, Title);
					if (DialogResponse != EAppReturnType::Ok)
					{
						return false;
					}
				}

				MinTimes.Reset();
				MaxTimes.Reset();
				AverageVertexData.Reset();
				AverageNormalData.Reset();

				MinTimes.AddZeroed(NumPolyMeshesToCompress);
				MaxTimes.AddZeroed(NumPolyMeshesToCompress);
				AverageVertexData.AddDefaulted(NumPolyMeshesToCompress);
				AverageNormalData.AddDefaulted(NumPolyMeshesToCompress);

				NumSamples = 0;
			}

			if (!AbcFile->ProcessFrames(IndividualMeshesFunc, Flags, SlowTask.Get()))
			{
				UE_LOGF(LogAbcImporter, Warning, "Alembic skeletal mesh import was interrupted");
				return false;
			}
			SlowTask.Reset();

			// Average out vertex data
			FBox AverageBoundingBox(ForceInit);
			const float Multiplier = 1.0f / static_cast<float>(FMath::Max(NumSamples, 1));
			for (TArray<FVector3f>& VertexData : AverageVertexData)
			{
				for (FVector3f& Vertex : VertexData)
				{
					Vertex *= Multiplier;
					AverageBoundingBox += (FVector)Vertex;
				}
			}
			const FVector AverageSampleCenter = AverageBoundingBox.GetCenter();

			TArray<TArray64<float>> Matrices;
			TArray<TArray64<float>> NormalsMatrices;
			for (int32 MeshIndex = 0; MeshIndex < NumPolyMeshesToCompress; ++MeshIndex)
			{
				Matrices.AddDefaulted();
				Matrices[MeshIndex].AddZeroed(int64(AverageVertexData[MeshIndex].Num()) * 3 * NumSamples);

				NormalsMatrices.AddDefaulted();
				NormalsMatrices[MeshIndex].AddZeroed(int64(AverageNormalData[MeshIndex].Num()) * 3 * NumSamples);
			}

			if (bEnableSamplesOffsets)
			{
				SamplesOffsets.Emplace();
				SamplesOffsets.GetValue().AddDefaulted(NumSamples);
			}

			if (bEnableSamplesOffsets)
			{
				SamplesOffsets.Emplace();
				SamplesOffsets.GetValue().AddDefaulted(NumSamples);
			}

			SlowTask = MakeUnique<FScopedSlowTask>(static_cast<float>((ImportSettings->SamplingSettings.FrameEnd + 1) - ImportSettings->SamplingSettings.FrameStart), FText::FromString(FString(TEXT("Generating matrices"))));
			SlowTask->MakeDialog(true);

			uint32 GenerateMatrixSampleIndex = 0;
			CompletedFrames = 0.0f;
			TFunction<void(int32, FAbcFile*)> GenerateMatrixFunc =
				[this, NumPolyMeshesToCompress, &Matrices, &NormalsMatrices, &GenerateMatrixSampleIndex, &PolyMeshesToCompress, &AverageVertexData, &AverageNormalData, &AverageSampleCenter, &SlowTask, &CompletedFrames]
				(int32 FrameIndex, FAbcFile* InFile)
				{
					// Compute on bounding box for the sample, which will include all the meshes
					FBox BoundingBox(ForceInit);

					for (int32 MeshIndex = 0; MeshIndex < NumPolyMeshesToCompress; ++MeshIndex)
					{
						FAbcPolyMesh* PolyMesh = PolyMeshesToCompress[MeshIndex];
						const uint32 NumMatrixRows = AverageVertexData[MeshIndex].Num() * 3;

						const TArray<FVector3f>& Vertices = PolyMesh->GetSample(FrameIndex)->Vertices;

						for ( const FVector3f& Vector : Vertices )
						{
							BoundingBox += (FVector)Vector;
						}
					}

					FVector SampleOffset = FVector::ZeroVector;
					if (SamplesOffsets.IsSet())
					{
						SampleOffset = BoundingBox.GetCenter() - AverageSampleCenter;
						SamplesOffsets.GetValue()[GenerateMatrixSampleIndex] = SampleOffset;
					}

					// For each object generate the delta frame data for the PCA compression
					for (int32 MeshIndex = 0; MeshIndex < NumPolyMeshesToCompress; ++MeshIndex)
					{
						FAbcPolyMesh* PolyMesh = PolyMeshesToCompress[MeshIndex];
						const uint32 NumMatrixRows = AverageVertexData[MeshIndex].Num() * 3;
						const TArray<FVector3f>& CurrentVertices = PolyMesh->GetSample(FrameIndex)->Vertices;
						const TArray<FVector3f>& CurrentNormals = PolyMesh->GetSample(FrameIndex)->Normals;

						const int32 AverageVertexOffset = 0;
						const int32 AverageIndexOffset = 0;

						AbcImporterUtilities::GenerateDeltaFrameDataMatrix(CurrentVertices, CurrentNormals, AverageVertexData[MeshIndex], AverageNormalData[MeshIndex],
							GenerateMatrixSampleIndex, AverageVertexOffset, AverageIndexOffset, SampleOffset, Matrices[MeshIndex], NormalsMatrices[MeshIndex]);
					}

					++GenerateMatrixSampleIndex;
					++CompletedFrames;

					if (IsInGameThread())
					{
						SlowTask->EnterProgressFrame(CompletedFrames);
						CompletedFrames = 0.0f;
					}					
				};

			if (!AbcFile->ProcessFrames(GenerateMatrixFunc, Flags, SlowTask.Get()))
			{
				UE_LOGF(LogAbcImporter, Warning, "Alembic skeletal mesh import was interrupted");
				return false;
			}
			SlowTask.Reset();

			for (int32 MeshIndex = 0; MeshIndex < NumPolyMeshesToCompress; ++MeshIndex)
			{
				// Perform compression
				TArray64<float> OutU, OutV, OutNormalsU;
				TArrayView64<float> BasesMatrix;
				TArrayView64<float> NormalsBasesMatrix;

				const int32 NumVertices = AverageVertexData[MeshIndex].Num();
				const int32 NumIndices = AverageNormalData[MeshIndex].Num();
				const int32 NumMatrixRows = NumVertices * 3;
				uint32 NumUsedSingularValues = NumSamples;

				// Allocate compressed mesh data object
				CompressedMeshData.AddDefaulted();
				FCompressedAbcData& CompressedData = CompressedMeshData.Last();
				CompressedData.AverageSample = new FAbcMeshSample(*PolyMeshesToCompress[MeshIndex]->GetTransformedFirstSample());
				FMemory::Memcpy(CompressedData.AverageSample->Vertices.GetData(), AverageVertexData[MeshIndex].GetData(), AverageVertexData[MeshIndex].GetTypeSize() * NumVertices);
				FMemory::Memcpy(CompressedData.AverageSample->Normals.GetData(), AverageNormalData[MeshIndex].GetData(), AverageNormalData[MeshIndex].GetTypeSize() * NumIndices);

				if ( InCompressionSettings.BaseCalculationType != EBaseCalculationType::NoCompression )
				{
					const float PercentageBases = InCompressionSettings.BaseCalculationType == EBaseCalculationType::PercentageBased ? InCompressionSettings.PercentageOfTotalBases / 100.0f : 1.0f;
					const int32 NumBases = InCompressionSettings.BaseCalculationType == EBaseCalculationType::FixedNumber ? InCompressionSettings.MaxNumberOfBases : 0;

					NumUsedSingularValues = PerformSVDCompression(Matrices[MeshIndex], NormalsMatrices[MeshIndex], NumSamples, PercentageBases, NumBases, OutU, OutNormalsU, OutV);
					BasesMatrix = OutU;
					NormalsBasesMatrix = OutNormalsU;
				}
				else
				{
					OutV.AddDefaulted(NumSamples * NumSamples);

					for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
					{
						for (int32 CurveIndex = 0; CurveIndex < NumSamples; ++CurveIndex)
						{
							float Weight = 0.f;

							if ( SampleIndex == CurveIndex )
							{
								Weight = 1.f;
							}

							OutV[SampleIndex + (NumSamples * CurveIndex )] = Weight;
						}
					}

					BasesMatrix = Matrices[MeshIndex];
					NormalsBasesMatrix = NormalsMatrices[MeshIndex];
				}

				const float FrameStep = (MaxTimes[MeshIndex] - MinTimes[MeshIndex]) / (float)(NumSamples - 1);
				AbcImporterUtilities::GenerateCompressedMeshData(CompressedData, NumUsedSingularValues, NumSamples, BasesMatrix, NormalsBasesMatrix, OutV, FrameStep, FMath::Max(MinTimes[MeshIndex], 0.0f));

				if (bRunComparison)
				{
					CompareCompressionResult(Matrices[MeshIndex], NumSamples, NumUsedSingularValues, BasesMatrix, OutV, THRESH_POINTS_ARE_SAME);
					CompareCompressionResult(NormalsMatrices[MeshIndex], NumSamples, NumUsedSingularValues, NormalsBasesMatrix, OutV, THRESH_NORMALS_ARE_SAME);
				}
			}
		}
	}
	else
	{
		bResult = ConstantPolyMeshObjects.Num() > 0;
		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(bResult ? EMessageSeverity::Warning : EMessageSeverity::Error, LOCTEXT("NoMeshesToProcess", "Unable to compress animation data, no meshes (with constant topology) found with Vertex Animation and baked Matrix Animation is turned off."));
		FAbcImportLogger::AddImportMessage(Message);
	}

	// Process the constant meshes by only adding them as average samples (without any bases/morphtargets to add as well)
	for (const FAbcPolyMesh* ConstantPolyMesh : ConstantPolyMeshObjects)
	{
		// Allocate compressed mesh data object
		CompressedMeshData.AddDefaulted();
		FCompressedAbcData& CompressedData = CompressedMeshData.Last();

		if (ImportSettings->CompressionSettings.bBakeMatrixAnimation)
		{
			CompressedData.AverageSample = new FAbcMeshSample(*ConstantPolyMesh->GetTransformedFirstSample());
		}
		else
		{
			CompressedData.AverageSample = new FAbcMeshSample(*ConstantPolyMesh->GetFirstSample());
		}
	}
		
	return bResult;
}

void FAbcImporter::CompareCompressionResult(const TArray64<float>& OriginalMatrix, const uint32 NumSamples, const uint32 NumUsedSingularValues, const TArrayView64<float>& OutU, const TArray64<float>& OutV, const float Tolerance)
{
	if (NumSamples == 0)
	{
		return;
	}

	const uint32 NumRows = IntCastChecked<uint32>(OriginalMatrix.Num() / NumSamples);

	TArray64<float> ComparisonMatrix;
	ComparisonMatrix.AddZeroed(OriginalMatrix.Num());
	for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
	{
		const int64 SampleOffset = (int64(SampleIndex) * NumRows);
		const int64 CurveOffset = (int64(SampleIndex) * NumUsedSingularValues);
		for (uint32 BaseIndex = 0; BaseIndex < NumUsedSingularValues; ++BaseIndex)
		{
			const int64 BaseOffset = (int64(BaseIndex) * NumRows);
			for (uint32 RowIndex = 0; RowIndex < NumRows; ++RowIndex)
			{
				ComparisonMatrix[RowIndex + SampleOffset] += OutU[RowIndex + BaseOffset] * OutV[BaseIndex + CurveOffset];
			}
		}
	}

	// Compare arrays
	for (int64 i = 0; i < ComparisonMatrix.Num(); ++i)
	{
		ensureMsgf(FMath::IsNearlyEqual(OriginalMatrix[i], ComparisonMatrix[i], Tolerance), TEXT("Difference of %2.10f found"), FMath::Abs(OriginalMatrix[i] - ComparisonMatrix[i]));
	}
}

const int32 FAbcImporter::PerformSVDCompression(const TArray64<float>& OriginalMatrix, const TArray64<float>& OriginalNormalsMatrix, const uint32 NumSamples, const float InPercentage, const int32 InFixedNumValue,
	TArray64<float>& OutU, TArray64<float>& OutNormalsU, TArray64<float>& OutV)
{
	FScopedSlowTask SlowTask(4.0f, FText::FromString(FString(TEXT("Decomposing animation"))));
	SlowTask.MakeDialog();

	const int32 NumRows = IntCastChecked<int32>(OriginalMatrix.Num() / NumSamples);

	TArray64<float> OutS;
	EigenHelpers::PerformSVD(OriginalMatrix, NumRows, NumSamples, OutU, OutV, OutS);
	SlowTask.EnterProgressFrame(1.0f);

	// Now we have the new basis data we have to construct the correct morph target data and curves
	const float PercentageBasesUsed = InPercentage;
	const int32 NumNonZeroSingularValues = IntCastChecked<int32>(OutS.Num());
	const int32 NumUsedSingularValues = (InFixedNumValue != 0) ? FMath::Min(InFixedNumValue, NumNonZeroSingularValues) : (int32)((float)NumNonZeroSingularValues * PercentageBasesUsed);

	// Pre-multiply the bases with it's singular values
	ParallelFor(NumUsedSingularValues, [&](int64 ValueIndex)
	{
		const float Multiplier = OutS[ValueIndex];
		const int64 ValueOffset = ValueIndex * NumRows;

		for (int32 RowIndex = 0; RowIndex < NumRows; ++RowIndex)
		{
			OutU[ValueOffset + RowIndex] *= Multiplier;
		}
	});

	// Project OriginalNormalsMatrix on OutV to deduce OutNormalsU
	// 
	// OriginalNormalsMatrix * OutV.transpose() = OutNormalsU
	//
	// This takes into account that OutNormalsU should be already scaled by what would be OutNormalsS, just like OutU is scaled by OutS

	const int32 NormalsNumRows = IntCastChecked<int32>(OriginalNormalsMatrix.Num() / NumSamples);

	Eigen::MatrixXf NormalsMatrix;
	EigenHelpers::ConvertArrayToEigenMatrix(OriginalNormalsMatrix, NormalsNumRows, NumSamples, NormalsMatrix);
	SlowTask.EnterProgressFrame(1.0f);

	const int32 OutVNumRows = IntCastChecked<uint32>(OutV.Num() / NumSamples);

	Eigen::MatrixXf VMatrix;
	EigenHelpers::ConvertArrayToEigenMatrix(OutV, OutVNumRows, NumSamples, VMatrix);
	SlowTask.EnterProgressFrame(1.0f);

	Eigen::MatrixXf NormalsUMatrix = NormalsMatrix * VMatrix.transpose();

	uint32 OutNumColumns, OutNumRows;
	EigenHelpers::ConvertEigenMatrixToArray(NormalsUMatrix, OutNormalsU, OutNumColumns, OutNumRows);
	SlowTask.EnterProgressFrame(1.0f);

	UE_LOGF(LogAbcImporter, Log, "Decomposed animation and reconstructed with %i number of bases (full %lli, percentage %f, calculated %i)", NumUsedSingularValues, OutS.Num(), PercentageBasesUsed * 100.0f, NumUsedSingularValues);	
	
	return NumUsedSingularValues;
}

const TArray<UStaticMesh*> FAbcImporter::ReimportAsStaticMesh(UStaticMesh* Mesh)
{
	const FString StaticMeshName = Mesh->GetName();
	const TArray<UStaticMesh*> StaticMeshes = ImportAsStaticMesh(Mesh->GetOuter(), RF_Public | RF_Standalone);

	return StaticMeshes;
}

UGeometryCache* FAbcImporter::ReimportAsGeometryCache(UGeometryCache* GeometryCache)
{
	UGeometryCache* ReimportedCache = ImportAsGeometryCache(GeometryCache->GetOuter(), RF_Public | RF_Standalone);
	return ReimportedCache;
}

TArray<UObject*> FAbcImporter::ReimportAsSkeletalMesh(USkeletalMesh* SkeletalMesh)
{
	TArray<UObject*> ReimportedObjects = ImportAsSkeletalMesh(SkeletalMesh->GetOuter(), RF_Public | RF_Standalone);
	return ReimportedObjects;
}

const TArray<FAbcPolyMesh*>& FAbcImporter::GetPolyMeshes() const
{
	return AbcFile->GetPolyMeshes();
}

const uint32 FAbcImporter::GetStartFrameIndex() const
{
	return (AbcFile != nullptr) ? AbcFile->GetMinFrameIndex() : 0;
}

const uint32 FAbcImporter::GetEndFrameIndex() const
{
	return (AbcFile != nullptr) ? FMath::Max(AbcFile->GetMaxFrameIndex() - 1, 1) : 1;
}

const uint32 FAbcImporter::GetNumMeshTracks() const
{
	return (AbcFile != nullptr) ? AbcFile->GetNumPolyMeshes() : 0;
}

void FAbcImporter::GenerateMeshDescriptionFromSample(const TArray<FString>& UniqueFaceSetNames, const TArray<int32>& LookupMaterialSlot, const FAbcMeshSample* Sample, FMeshDescription* MeshDescription)
{
	if (MeshDescription == nullptr)
	{
		return;
	}

	FStaticMeshAttributes Attributes(*MeshDescription);

	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

	//Speedtree use UVs to store is data
	VertexInstanceUVs.SetNumChannels(Sample->NumUVSets);

	for (const FString& FaceSetName: UniqueFaceSetNames)
	{
		const FPolygonGroupID PolygonGroupID = MeshDescription->CreatePolygonGroup();
		PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = FName(*FaceSetName);
	}

	// position
	for (int32 VertexIndex = 0; VertexIndex < Sample->Vertices.Num(); ++VertexIndex)
	{
		FVector3f Position = Sample->Vertices[VertexIndex];

		FVertexID VertexID = MeshDescription->CreateVertex();
		VertexPositions[VertexID] = FVector3f(Position);
	}

	uint32 VertexIndices[3];
	uint32 TriangleCount = Sample->Indices.Num() / 3;
	for (uint32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
	{
		const uint32 IndiceIndex0 = TriangleIndex * 3;
		VertexIndices[0] = Sample->Indices[IndiceIndex0];
		VertexIndices[1] = Sample->Indices[IndiceIndex0 + 1];
		VertexIndices[2] = Sample->Indices[IndiceIndex0 + 2];

		// Skip degenerated triangle
		if (VertexIndices[0] == VertexIndices[1] || VertexIndices[1] == VertexIndices[2] || VertexIndices[0] == VertexIndices[2])
		{
			continue;
		}

		TArray<FVertexInstanceID> CornerVertexInstanceIDs;
		CornerVertexInstanceIDs.SetNum(3);
		FVertexID CornerVertexIDs[3];
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			uint32 IndiceIndex = IndiceIndex0 + Corner;
			uint32 VertexIndex = VertexIndices[Corner];
			const FVertexID VertexID(VertexIndex);
			const FVertexInstanceID VertexInstanceID = MeshDescription->CreateVertexInstance(VertexID);

			// tangents
			FVector3f TangentX = Sample->TangentX[IndiceIndex];
			FVector3f TangentY = Sample->TangentY[IndiceIndex];
			FVector3f TangentZ = Sample->Normals[IndiceIndex];

			VertexInstanceTangents[VertexInstanceID] = TangentX;
			VertexInstanceNormals[VertexInstanceID] = TangentZ;
			VertexInstanceBinormalSigns[VertexInstanceID] = GetBasisDeterminantSign((FVector)TangentX.GetSafeNormal(), (FVector)TangentY.GetSafeNormal(), (FVector)TangentZ.GetSafeNormal());

			if (Sample->Colors.Num())
			{
				VertexInstanceColors[VertexInstanceID] = FVector4f(Sample->Colors[IndiceIndex]);
			}
			else
			{
				VertexInstanceColors[VertexInstanceID] = FVector4f(FLinearColor::White);
			}

			for (uint32 UVIndex = 0; UVIndex < Sample->NumUVSets; ++UVIndex)
			{
				VertexInstanceUVs.Set(VertexInstanceID, UVIndex, Sample->UVs[UVIndex][IndiceIndex]);
			}
			CornerVertexInstanceIDs[Corner] = VertexInstanceID;
			CornerVertexIDs[Corner] = VertexID;
		}

		int32 MaterialSlotID = LookupMaterialSlot[Sample->MaterialIndices[TriangleIndex]];
		const FPolygonGroupID PolygonGroupID(MaterialSlotID);

		// Insert a polygon into the mesh
		MeshDescription->CreatePolygon(PolygonGroupID, CornerVertexInstanceIDs);
	}
	//Set the edge hardness from the smooth group
	FStaticMeshOperations::ConvertSmoothGroupToHardEdges(Sample->SmoothingGroupIndices, *MeshDescription);
}

bool FAbcImporter::BuildSkeletalMesh( FSkeletalMeshLODModel& LODModel, const FReferenceSkeleton& RefSkeleton, FAbcMeshSample* Sample, 
	int32 NumMaterialSlots, const TArray<int32> LookupMaterialSlot, 
	TArray<int32>& OutMorphTargetVertexRemapping, TArray<int32>& OutUsedVertexIndicesForMorphs)
{
	// Module manager is not thread safe, so need to prefetch before parallelfor
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");

	const bool bComputeNormals = (Sample->Normals.Num() == 0);
	const bool bComputeTangents = (Sample->TangentX.Num() == 0) || (Sample->TangentY.Num() == 0);

	// Compute normals/tangents if needed
	if (bComputeNormals || bComputeTangents)
	{
		uint32 TangentOptions = 0;
		MeshUtilities.CalculateTangents(Sample->Vertices, Sample->Indices, Sample->UVs[0], Sample->SmoothingGroupIndices, TangentOptions, Sample->TangentX, Sample->TangentY, Sample->Normals);
	}

	// Populate faces
	const uint32 NumFaces = Sample->Indices.Num() / 3;
	TArray<SkeletalMeshImportData::FMeshFace> Faces;
	Faces.AddZeroed(NumFaces);

	TArray<FMeshSection> MeshSections;
	MeshSections.AddDefaulted(NumMaterialSlots);

	// Process all the faces and add to their respective mesh section
	for (uint32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
	{
		const uint32 FaceOffset = FaceIndex * 3;
		const int32 MaterialIndex = LookupMaterialSlot[Sample->MaterialIndices[FaceIndex]];

		check(MeshSections.IsValidIndex(MaterialIndex));

		FMeshSection& Section = MeshSections[MaterialIndex];
		Section.MaterialIndex = MaterialIndex;
		Section.NumUVSets = Sample->NumUVSets;
	
		for (uint32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
		{
			LODModel.MaxImportVertex = FMath::Max<int32>(LODModel.MaxImportVertex, Sample->Indices[FaceOffset + VertexIndex]);

			Section.OriginalIndices.Add(FaceOffset + VertexIndex);
			Section.Indices.Add(Sample->Indices[FaceOffset + VertexIndex]);
			Section.TangentX.Add((FVector)Sample->TangentX[FaceOffset + VertexIndex]);
			Section.TangentY.Add((FVector)Sample->TangentY[FaceOffset + VertexIndex]);
			Section.TangentZ.Add((FVector)Sample->Normals[FaceOffset + VertexIndex]);

			for (uint32 UVIndex = 0; UVIndex < Sample->NumUVSets; ++UVIndex)
			{
				Section.UVs[UVIndex].Add(FVector2D(Sample->UVs[UVIndex][FaceOffset + VertexIndex]));
			}		
			
			Section.Colors.Add(Sample->Colors[FaceOffset + VertexIndex].ToFColor(false));
		}

		++Section.NumFaces;
	}

	// Sort the vertices by z value
	MeshSections.Sort([](const FMeshSection& A, const FMeshSection& B) { return A.MaterialIndex < B.MaterialIndex; });

	// Create Skeletal mesh LOD sections
	LODModel.Sections.Empty(MeshSections.Num());
	LODModel.NumVertices = 0;
	LODModel.IndexBuffer.Empty();

	TArray<uint32>& RawPointIndices = LODModel.GetRawPointIndices();
	RawPointIndices.Reset();

	TArray< TArray<uint32> > VertexIndexRemap;
	VertexIndexRemap.Empty(MeshSections.Num());

	// Create actual skeletal mesh sections
	for (int32 SectionIndex = 0; SectionIndex < MeshSections.Num(); ++SectionIndex)
	{
		const FMeshSection& SourceSection = MeshSections[SectionIndex];
		FSkelMeshSection& TargetSection = *new(LODModel.Sections) FSkelMeshSection();
		TargetSection.MaterialIndex = (uint16)SourceSection.MaterialIndex;
		TargetSection.NumTriangles = SourceSection.NumFaces;
		TargetSection.BaseVertexIndex = LODModel.NumVertices;

		// Separate the section's vertices into rigid and soft vertices.
		TArray<uint32>& ChunkVertexIndexRemap = *new(VertexIndexRemap)TArray<uint32>();
		ChunkVertexIndexRemap.AddUninitialized(SourceSection.NumFaces * 3);

		TMultiMap<uint32, uint32> FinalVertices;
		TArray<uint32> DuplicateVertexIndices;

		// Reused soft vertex
		FSoftSkinVertex NewVertex;

		uint32 VertexOffset = 0;
		// Generate Soft Skin vertices (used by the skeletal mesh)
		for (uint32 FaceIndex = 0; FaceIndex < SourceSection.NumFaces; ++FaceIndex)
		{
			const uint32 FaceOffset = FaceIndex * 3;

			for (uint32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
			{
				const uint32 Index = SourceSection.Indices[FaceOffset + VertexIndex];

				DuplicateVertexIndices.Reset();
				FinalVertices.MultiFind(Index, DuplicateVertexIndices);
				
				// Populate vertex data
				NewVertex.Position = Sample->Vertices[Index];
				NewVertex.TangentX = (FVector3f)SourceSection.TangentX[FaceOffset + VertexIndex];
				NewVertex.TangentY = (FVector3f)SourceSection.TangentY[FaceOffset + VertexIndex];
				NewVertex.TangentZ = (FVector3f)SourceSection.TangentZ[FaceOffset + VertexIndex]; // LWC_TODO: precision loss
				for (uint32 UVIndex = 0; UVIndex < SourceSection.NumUVSets; ++UVIndex)
				{
					NewVertex.UVs[UVIndex] = FVector2f(SourceSection.UVs[UVIndex][FaceOffset + VertexIndex]);
				}
				
				NewVertex.Color = SourceSection.Colors[FaceOffset + VertexIndex];

				// Set up bone influence (only using one bone so maxed out weight)
				FMemory::Memzero(NewVertex.InfluenceBones);
				FMemory::Memzero(NewVertex.InfluenceWeights);
				NewVertex.InfluenceWeights[0] = UE::AnimationCore::MaxRawBoneWeight;
				
				int32 FinalVertexIndex = INDEX_NONE;
				if (DuplicateVertexIndices.Num())
				{
					for (const uint32 DuplicateVertexIndex : DuplicateVertexIndices)
					{
						if (AbcImporterUtilities::AreVerticesEqual(TargetSection.SoftVertices[DuplicateVertexIndex], NewVertex))
						{
							// Use the existing vertex
							FinalVertexIndex = DuplicateVertexIndex;
							break;
						}
					}
				}

				if (FinalVertexIndex == INDEX_NONE)
				{
					FinalVertexIndex = TargetSection.SoftVertices.Add(NewVertex);
#if PRINT_UNIQUE_VERTICES
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Vert - P(%.2f, %.2f,%.2f) N(%.2f, %.2f,%.2f) TX(%.2f, %.2f,%.2f) TY(%.2f, %.2f,%.2f) UV(%.2f, %.2f)\n"), NewVertex.Position.X, NewVertex.Position.Y, NewVertex.Position.Z, SourceSection.TangentX[FaceOffset + VertexIndex].X, 
						SourceSection.TangentZ[FaceOffset + VertexIndex].X, SourceSection.TangentZ[FaceOffset + VertexIndex].Y, SourceSection.TangentZ[FaceOffset + VertexIndex].Z, SourceSection.TangentX[FaceOffset + VertexIndex].Y, SourceSection.TangentX[FaceOffset + VertexIndex].Z, SourceSection.TangentY[FaceOffset + VertexIndex].X, SourceSection.TangentY[FaceOffset + VertexIndex].Y, SourceSection.TangentY[FaceOffset + VertexIndex].Z, NewVertex.UVs[0].X, NewVertex.UVs[0].Y);
#endif

					FinalVertices.Add(Index, FinalVertexIndex);
					OutUsedVertexIndicesForMorphs.Add(Index);
					OutMorphTargetVertexRemapping.Add(SourceSection.OriginalIndices[FaceOffset + VertexIndex]);
				}

				RawPointIndices.Add(FinalVertexIndex);
				ChunkVertexIndexRemap[VertexOffset] = TargetSection.BaseVertexIndex + FinalVertexIndex;
				++VertexOffset;
			}
		}

		LODModel.NumVertices += TargetSection.SoftVertices.Num();
		TargetSection.NumVertices = TargetSection.SoftVertices.Num();

		// Only need first bone from active bone indices
		TargetSection.BoneMap.Add(0);

		TargetSection.CalcMaxBoneInfluences();
		TargetSection.CalcUse16BitBoneIndex();
	}

	// Only using bone zero
	LODModel.ActiveBoneIndices.Add(0);

	// Finish building the sections.
	for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
	{
		FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

		const TArray<uint32>& SectionIndices = MeshSections[SectionIndex].Indices;
		Section.BaseIndex = LODModel.IndexBuffer.Num();
		const int32 NumIndices = SectionIndices.Num();
		const TArray<uint32>& SectionVertexIndexRemap = VertexIndexRemap[SectionIndex];
		for (int32 Index = 0; Index < NumIndices; Index++)
		{
			uint32 VertexIndex = SectionVertexIndexRemap[Index];
			LODModel.IndexBuffer.Add(VertexIndex);
		}
	}

	// Compute the required bones for this model.
	USkeletalMesh::CalculateRequiredBones(LODModel, RefSkeleton, nullptr);

	return true;
}

void FAbcImporter::GenerateMorphTargetVertices(FAbcMeshSample* BaseSample, TArray<FMorphTargetDelta> &MorphDeltas, FAbcMeshSample* AverageSample, uint32 WedgeOffset, const TArray<int32>& RemapIndices, const TArray<int32>& UsedVertexIndicesForMorphs, const uint32 VertexOffset, const uint32 IndexOffset)
{
	FMorphTargetDelta MorphVertex;
	const uint32 NumberOfUsedVertices = UsedVertexIndicesForMorphs.Num();	
	for (uint32 VertIndex = 0; VertIndex < NumberOfUsedVertices; ++VertIndex)
	{
		const int32 UsedVertexIndex = UsedVertexIndicesForMorphs[VertIndex] - VertexOffset;
		const uint32 UsedNormalIndex = RemapIndices[VertIndex] - IndexOffset;

		if (UsedVertexIndex >= 0 && UsedVertexIndex < BaseSample->Vertices.Num())
		{
			// Position delta
			MorphVertex.PositionDelta = BaseSample->Vertices[UsedVertexIndex] - AverageSample->Vertices[UsedVertexIndex];
			// Tangent delta
			MorphVertex.TangentZDelta = BaseSample->Normals[UsedNormalIndex] - AverageSample->Normals[UsedNormalIndex];
			// Index of base mesh vert this entry is to modify
			MorphVertex.SourceIdx = VertIndex;
			MorphDeltas.Add(MorphVertex);
		}
	}
}

FCompressedAbcData::~FCompressedAbcData()
{
	delete AverageSample;
	for (FAbcMeshSample* Sample : BaseSamples)
	{
		delete Sample;
	}
}

#undef LOCTEXT_NAMESPACE
