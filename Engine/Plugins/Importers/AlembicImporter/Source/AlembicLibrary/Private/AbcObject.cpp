// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcObject.h"

IAbcObject::IAbcObject(const Alembic::Abc::IObject& InObject, const FAbcFile* InFile, IAbcObject* InParent) : Parent(InParent), File(InFile), Object(InObject), bConstant(false), MinTime(TNumericLimits<float>::Max())
, MaxTime(TNumericLimits<float>::Min()), StartFrameIndex(0), NumSamples(0)
{
	Name = FString(Object.getName().c_str());

	for (int32 Index = 0; Index < MaxNumberOfResidentSamples; ++Index)
	{
		FrameTimes[Index] = 0.f;
		ResidentSampleIndices[Index] = INDEX_NONE;
		InUseSamples[Index] = false;
	}
}

float IAbcObject::GetTimeForFrameIndex(const int32 FrameIndex) const
{
	for (int32 Index = 0; Index < MaxNumberOfResidentSamples; ++Index)
	{
		if (ResidentSampleIndices[Index] == FrameIndex)
		{
			return FrameTimes[Index];
		}
	}

	return 0.0f;
}
