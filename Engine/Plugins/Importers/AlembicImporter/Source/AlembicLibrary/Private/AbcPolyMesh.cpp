// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcPolyMesh.h"
#include "AbcImportSettings.h"
#include "AbcImportUtilities.h"

#include "AbcFile.h"

THIRD_PARTY_INCLUDES_START
#include <Alembic/AbcGeom/Visibility.h>
THIRD_PARTY_INCLUDES_END

static const ESampleReadFlags ReadAllFlags = ESampleReadFlags::Positions | ESampleReadFlags::Indices | ESampleReadFlags::UVs | ESampleReadFlags::Normals |
	ESampleReadFlags::Colors | ESampleReadFlags::MaterialIndices | ESampleReadFlags::Velocities;

FAbcPolyMesh::FAbcPolyMesh(const Alembic::AbcGeom::IPolyMesh& InPolyMesh, const FAbcFile* InFile, IAbcObject* InParent /*= nullptr*/)
	: IAbcObject(InPolyMesh, InFile, InParent), SelfBounds(EForceInit::ForceInitToZero), ChildBounds(EForceInit::ForceInitToZero), bShouldImport(true), PolyMesh(InPolyMesh),
	  Schema(InPolyMesh.getSchema()), FirstSample(nullptr), TransformedFirstSample(nullptr),
	  SampleReadFlags(ReadAllFlags), bReturnFirstSample(false), bReturnTransformedFirstSample(false), bFirstFrameVisibility(true)
{
	// Retrieve schema and frame information		
	NumSamples = Schema.getNumSamples();
	bConstant = Schema.isConstant();
	bConstantTopology = (Schema.getTopologyVariance() != Alembic::AbcGeom::kHeterogeneousTopology) || bConstant;
	bConstantVisibility = AbcImporterUtilities::IsObjectVisibilityConstant(InPolyMesh);
	SelfBounds = AbcImporterUtilities::ExtractBounds(Schema.getSelfBoundsProperty());
	ChildBounds = AbcImporterUtilities::ExtractBounds(Schema.getChildBoundsProperty());
	
	// Retrieve min and max time/frames information 
	AbcImporterUtilities::GetMinAndMaxTime(Schema, MinTime, MaxTime);
	AbcImporterUtilities::GetStartTimeAndFrame(Schema, MinTime, StartFrameIndex);

	/** Yuck, but retrieving face set is actually a file read operation as they are not cached on the schema (only function that is non-const) */
	AbcImporterUtilities::RetrieveFaceSetNames(*const_cast<Alembic::AbcGeom::IPolyMeshSchema*>(&Schema), FaceSetNames);

	bConstantTransformation = Parent ? Parent->HasConstantTransform() : true;

	for (int32 Index = 0; Index < MaxNumberOfResidentSamples; ++Index)
	{
		ResidentSamples[Index] = new FAbcMeshSample();
		ResidentVisibilitySamples[Index] = true;
	}
}

bool FAbcPolyMesh::ReadFirstFrame(const float InTime, const int32 FrameIndex)
{
	checkf(FirstSample == nullptr, TEXT("Reading First Frame Twice"));

	// Only try to read velocities when importing a geometry cache with ImportAbcVelocitiesAsMotionVectors
	const bool bReadVelocities = File->GetImportSettings()->ImportType == EAlembicImportType::GeometryCache &&
		File->GetImportSettings()->GeometryCacheSettings.MotionVectors == EAbcGeometryCacheMotionVectorsImport::ImportAbcVelocitiesAsMotionVectors;

	if (!bReadVelocities)
	{
		EnumRemoveFlags(SampleReadFlags, ESampleReadFlags::Velocities);
	}

	const float Time = InTime < MinTime ? MinTime : (InTime > MinTime ? MinTime : InTime);
	Alembic::Abc::ISampleSelector SampleSelector = AbcImporterUtilities::GenerateAlembicSampleSelector<double>(Time);
	const bool bFirstFrame = true;
	FirstSample = AbcImporterUtilities::GenerateAbcMeshSampleForFrame(Schema, SampleSelector, SampleReadFlags, bFirstFrame);
	bFirstFrameVisibility = AbcImporterUtilities::IsObjectVisible(PolyMesh, SampleSelector);
	SampleReadFlags = AbcImporterUtilities::GenerateAbcMeshSampleReadFlags(Schema);
	// Remove the velocities flag again since GenerateAbcMeshSampleReadFlags generates a new SampleReadFlags from scratch
	if (!bReadVelocities)
	{
		EnumRemoveFlags(SampleReadFlags, ESampleReadFlags::Velocities);
	}

	if (FirstSample)
	{
		if (FirstSample->Normals.Num() == 0)
		{
			// Normals are not available so they should be calculated each frame, so force set the Normals read flag which will ensure they are generated
			SampleReadFlags |= ESampleReadFlags::Normals;
		}	

		CalculateNormalsForSample(FirstSample);

		// Compute tangents for mesh
		AbcImporterUtilities::ComputeTangents(FirstSample, File->GetImportSettings()->NormalGenerationSettings.bIgnoreDegenerateTriangles, *File->GetMeshUtilities());

		const bool bApplyTransformation = (File->GetImportSettings()->ImportType == EAlembicImportType::StaticMesh && File->GetImportSettings()->StaticMeshSettings.bPropagateMatrixTransformations) || (File->GetImportSettings()->ImportType == EAlembicImportType::Skeletal && File->GetImportSettings()->CompressionSettings.bBakeMatrixAnimation) || File->GetImportSettings()->ImportType == EAlembicImportType::GeometryCache;

		// Transform copy of the first sample 
		TransformedFirstSample = new FAbcMeshSample(*FirstSample);
		FMatrix HierarchyTransform = GetMatrix(FrameIndex);
		AbcImporterUtilities::ApplyConversion(HierarchyTransform, File->GetImportSettings()->ConversionSettings);
		AbcImporterUtilities::PropogateMatrixTransformationToSample(TransformedFirstSample, HierarchyTransform);
		AbcImporterUtilities::ApplyConversion(TransformedFirstSample, File->GetImportSettings()->ConversionSettings, true);

		if (bConstant && bConstantTransformation && !bApplyTransformation)
		{
			bReturnFirstSample = true;
		}
		else if (bConstant && bConstantTransformation)
		{
			bReturnTransformedFirstSample = true;
		}
		else
		{
			// Resident samples from initial sample (this copies all vertex attributes, which allows us to skip re-reading constant attributes in future sample reads)
			for (int32 Index = 0; Index < MaxNumberOfResidentSamples; ++Index)
			{
				ResidentSamples[Index]->Copy(FirstSample, SampleReadFlags);
			}
		}
	}	

	return FirstSample != nullptr;
}

void FAbcPolyMesh::CalculateNormalsForSample(FAbcMeshSample* Sample)
{
	/*
	Normal cases
	* No normals
	- OneSmoothing group -> smooth normals and zeroed out smoothing groups
	- Compute smooth normals
	* Normals
	- OneSmoothing group -> smooth normals and zeroed out smoothing groups
	- Recompute normals -> Compute normals, compute smoothing groups -> compute smooth normals
	- else compute smoothing groups
	*/

	const bool bRecomputeNormals = File->GetImportSettings()->NormalGenerationSettings.bRecomputeNormals;
	if (File->GetImportSettings()->NormalGenerationSettings.bForceOneSmoothingGroupPerObject && bRecomputeNormals)
	{
		AbcImporterUtilities::CalculateSmoothNormals(Sample);
		Sample->SmoothingGroupIndices.AddZeroed(Sample->Indices.Num() / 3);
		Sample->NumSmoothingGroups = 1;
	}
	else
	{
		const bool bNormalsAvailable = Sample->Normals.Num() != 0;

		// Recompute the (hard) normals if the user opted to do so
		if (bRecomputeNormals)
		{
			AbcImporterUtilities::CalculateNormals(Sample);
		}
		// Otherwise we'd expect normals to be available, if not we should assume the object has smooth normals (so calculate them)
		else if (!bNormalsAvailable)
		{			
			AbcImporterUtilities::CalculateSmoothNormals(Sample);
			Sample->SmoothingGroupIndices.AddZeroed(Sample->Indices.Num() / 3);
			Sample->NumSmoothingGroups = 1;
		}

		// If not smooth normals generate smoothing groups
		if (bNormalsAvailable || bRecomputeNormals)
		{
			// Generate smoothing groups from the normals to use for following samples
			AbcImporterUtilities::GenerateSmoothingGroupsIndices(Sample, File->GetImportSettings()->NormalGenerationSettings.HardEdgeAngleThreshold);
		}

		// In case we are expected to recompute the normals now recalculate the normals using the calculated smoothing groups
		if (bRecomputeNormals)
		{
			AbcImporterUtilities::CalculateNormalsWithSmoothingGroups(Sample, Sample->SmoothingGroupIndices, Sample->NumSmoothingGroups);
		}
	}
}

void FAbcPolyMesh::SetFrameAndTime(const float InTime, const int32 FrameIndex, const EFrameReadFlags InFlags, const int32 TargetIndex /*= INDEX_NONE*/)
{
	if (!bShouldImport)
	{
		return;
	}

	// Generate mesh sample data from the Alembic Poly Mesh Schema
	Alembic::Abc::ISampleSelector SampleSelector = AbcImporterUtilities::GenerateAlembicSampleSelector<double>(InTime);

	// Read frame data for new time
	if (TargetIndex != INDEX_NONE)
	{
		InUseSamples[TargetIndex] = true;
		ResidentSampleIndices[TargetIndex] = FrameIndex;
		FAbcMeshSample* WriteSample = ResidentSamples[TargetIndex];
		checkf(WriteSample != nullptr, TEXT("Samples not initialized"));
		FrameTimes[TargetIndex] = InTime;

		if (!bConstant)
		{
			const bool bRecomputeNormals = File->GetImportSettings()->NormalGenerationSettings.bRecomputeNormals;
			const ESampleReadFlags SampleReadFlagsOverride = EnumHasAnyFlags(InFlags, EFrameReadFlags::PositionAndNormalOnly) ? ESampleReadFlags::Positions |  ESampleReadFlags::Normals : SampleReadFlags;
			WriteSample->Copy(FirstSample, SampleReadFlagsOverride);
			const bool bValidSample = AbcImporterUtilities::GenerateAbcMeshSampleDataForFrame(Schema, SampleSelector, WriteSample, SampleReadFlagsOverride, InTime == MinTime);
			if (!bValidSample)
			{
				return;
			}
			// Check whether or not the number of normal indices matches with the first frame
			const bool bMatchingIndices = FirstSample != nullptr && FirstSample->Indices.Num() == WriteSample->Indices.Num();
			// Make sure in case of recomputing normals we enforece using the first sample data (otherwise we'll be using loaded or incorrectly calculated normals)
			if (WriteSample->Normals.Num() == 0 || bRecomputeNormals)
			{
				// If the indices match we can recalculate the normals according to the first frame (and copy the smoothing indices)
				if (bMatchingIndices)
				{
					AbcImporterUtilities::CalculateNormalsWithSampleData(WriteSample, FirstSample);
				}
				else
				{
					CalculateNormalsForSample(WriteSample);
				}
			}
			else
			{
				// Check the user-configurable setting for smoothing group generation
				if (!File->GetImportSettings()->NormalGenerationSettings.bForceOneSmoothingGroupPerObject)
				{
					AbcImporterUtilities::GenerateSmoothingGroupsIndices(WriteSample, File->GetImportSettings()->NormalGenerationSettings.HardEdgeAngleThreshold);
				}
				else
				{
					const int32 NumFaces = WriteSample->Indices.Num() / 3;
					WriteSample->SmoothingGroupIndices.Empty(NumFaces);
					WriteSample->SmoothingGroupIndices.AddZeroed(NumFaces);
					WriteSample->NumSmoothingGroups = 1;
				}
			}

			// Computing tangents skippable for GeometryCache only, the other import types need tangents
			if (File->GetImportSettings()->ImportType != EAlembicImportType::GeometryCache ||
				!File->GetImportSettings()->NormalGenerationSettings.bSkipComputingTangents)
			{
				AbcImporterUtilities::ComputeTangents(WriteSample, File->GetImportSettings()->NormalGenerationSettings.bIgnoreDegenerateTriangles, *File->GetMeshUtilities());
			}
		}
		else if (bConstant && !bConstantTransformation)
		{
			// In this case FirstSample is the only sample, so we just copy it and apply the current matrix 
			WriteSample->Copy(FirstSample, ESampleReadFlags::Default);
		}

		if (EnumHasAnyFlags(InFlags, EFrameReadFlags::ApplyMatrix))
		{
			FMatrix HierarchyTransform = GetMatrix(FrameIndex);
			AbcImporterUtilities::ApplyConversion(HierarchyTransform, File->GetImportSettings()->ConversionSettings);
			AbcImporterUtilities::PropogateMatrixTransformationToSample(WriteSample, HierarchyTransform);
			AbcImporterUtilities::ApplyConversion(WriteSample, File->GetImportSettings()->ConversionSettings, true);
		}

		if (!bConstantVisibility)
		{
			ResidentVisibilitySamples[TargetIndex] = AbcImporterUtilities::IsObjectVisible(PolyMesh, SampleSelector);
		}
	}
}

FMatrix FAbcPolyMesh::GetMatrix(const int32 FrameIndex) const
{
	FMatrix DefaultMatrix = FMatrix::Identity;
	AbcImporterUtilities::ApplyConversion(DefaultMatrix, File->GetImportSettings()->ConversionSettings);
	return Parent ? Parent->GetMatrix(FrameIndex) : DefaultMatrix;
}

bool FAbcPolyMesh::HasConstantTransform() const
{
	return bConstantTransformation;
}

void FAbcPolyMesh::PurgeFrameData(const int32 ReadIndex)
{
	if (bShouldImport)
	{
		checkf(InUseSamples[ReadIndex], TEXT("Trying to purge a sample which isn't in use"));
		InUseSamples[ReadIndex] = false;
		ResidentSampleIndices[ReadIndex] = INDEX_NONE;
	}
}

const FAbcMeshSample* FAbcPolyMesh::GetSample(const int32 FrameIndex) const
{
	if (bReturnFirstSample)
	{
		return FirstSample;
	}
	else if (bReturnTransformedFirstSample)
	{
		return TransformedFirstSample;
	}

	// Find sample within resident samples
	for (int32 Index = 0; Index < MaxNumberOfResidentSamples; ++Index)
	{
		if (ResidentSampleIndices[Index] == FrameIndex)
		{
			return ResidentSamples[Index];
		}
	}

	return nullptr;
}

const FAbcMeshSample* FAbcPolyMesh::GetFirstSample() const
{
	return FirstSample;
}

const FAbcMeshSample* FAbcPolyMesh::GetTransformedFirstSample() const
{
	return TransformedFirstSample;
}

ESampleReadFlags FAbcPolyMesh::GetSampleReadFlags() const
{
	return SampleReadFlags;
}

const bool FAbcPolyMesh::GetVisibility(const int32 FrameIndex) const
{
	if (bConstantVisibility)
	{
		return bFirstFrameVisibility;
	}

	// Find sample within resident samples
	for (int32 Index = 0; Index < MaxNumberOfResidentSamples; ++Index)
	{
		if (ResidentSampleIndices[Index] == FrameIndex)
		{
			return ResidentVisibilitySamples[Index];
		}
	}

	return true;
}

void FAbcMeshSample::Reset(const ESampleReadFlags ReadFlags)
{
	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Positions))
	{
		Vertices.SetNum(0, EAllowShrinking::No);
	}

	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Velocities))
	{
		Velocities.SetNum(0, EAllowShrinking::No);
	}

	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Indices))
	{
		Indices.SetNum(0, EAllowShrinking::No);
	}

	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Normals))
	{
		Normals.SetNum(0, EAllowShrinking::No);
		TangentX.SetNum(0, EAllowShrinking::No);
		TangentY.SetNum(0, EAllowShrinking::No);
	}

	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::UVs))
	{
		for (uint32 UVIndex = 0; UVIndex < MAX_TEXCOORDS; ++UVIndex)
		{
			UVs[UVIndex].SetNum(0, EAllowShrinking::No);
		}
	}

	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Colors))
	{
		Colors.SetNum(0, EAllowShrinking::No);
	}

	if (EnumHasAnyFlags(ReadFlags, ESampleReadFlags::MaterialIndices))
	{
		MaterialIndices.SetNum(0, EAllowShrinking::No);
	}

	SmoothingGroupIndices.SetNum(0, EAllowShrinking::No);
	NumSmoothingGroups = 0;
	NumMaterials = 0;
	SampleTime = 0.0f;
	NumUVSets = 1;
}

void FAbcMeshSample::Copy(const FAbcMeshSample& InSample, const ESampleReadFlags ReadFlags)
{
	Copy(&InSample, ReadFlags);
}

void FAbcMeshSample::Copy(const FAbcMeshSample* InSample, const ESampleReadFlags ReadFlags)
{
	Reset(ReadFlags);

	if (!EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Positions))
	{
		Vertices = InSample->Vertices;
	}

	if (!EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Velocities))
	{
		Velocities = InSample->Velocities;
	}

	if (!EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Indices))
	{
		Indices = InSample->Indices;
	}

	if (!EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Normals))
	{
		Normals = InSample->Normals;
		TangentX = InSample->TangentX;
		TangentY = InSample->TangentY;
		
		SmoothingGroupIndices = InSample->SmoothingGroupIndices;
		NumSmoothingGroups = InSample->NumSmoothingGroups;
	}

	if (!EnumHasAnyFlags(ReadFlags, ESampleReadFlags::UVs))
	{
		for (uint32 UVIndex = 0; UVIndex < InSample->NumUVSets; ++UVIndex)
		{
			UVs[UVIndex] = InSample->UVs[UVIndex];
		}
		NumUVSets = InSample->NumUVSets;
	}

	if (!EnumHasAnyFlags(ReadFlags, ESampleReadFlags::Colors))
	{
		Colors = InSample->Colors;
	}
	
	if (!EnumHasAnyFlags(ReadFlags, ESampleReadFlags::MaterialIndices))
	{
		MaterialIndices = InSample->MaterialIndices;
		NumMaterials = InSample->NumMaterials;
	}
	
	SampleTime = InSample->SampleTime;
}
