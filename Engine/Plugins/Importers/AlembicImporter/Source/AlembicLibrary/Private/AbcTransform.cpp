// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcTransform.h"
#include "AbcImportUtilities.h"


THIRD_PARTY_INCLUDES_START
#include <Alembic/AbcGeom/Visibility.h>
THIRD_PARTY_INCLUDES_END

FAbcTransform::FAbcTransform(const Alembic::AbcGeom::IXform& InTransform, const FAbcFile* InFile, IAbcObject* InParent /*= nullptr*/) : IAbcObject(InTransform, InFile, InParent), Transform(InTransform), Schema(InTransform.getSchema())
{
	NumSamples = Schema.getNumSamples();
	bConstant = Schema.isConstant();
	bConstantIdentity = Schema.isConstantIdentity();
	InitialValue = FMatrix::Identity;

	for (int32 Index = 0; Index < MaxNumberOfResidentSamples; ++Index)
	{
		ResidentMatrices[Index] = InitialValue;
	}
	
	// Retrieve min and max time/frames information 
	AbcImporterUtilities::GetMinAndMaxTime(Schema, MinTime, MaxTime);
	AbcImporterUtilities::GetStartTimeAndFrame(Schema, MinTime, StartFrameIndex);
}

bool FAbcTransform::ReadFirstFrame(const float InTime, const int32 FrameIndex)
{
	// Read matrix sample for sample time
	const float Time = InTime < MinTime ? MinTime : (InTime > MinTime ? MinTime : InTime);
	Alembic::Abc::ISampleSelector SampleSelector = AbcImporterUtilities::GenerateAlembicSampleSelector<double>(Time);
	Alembic::AbcGeom::XformSample MatrixSample;
	Schema.get(MatrixSample, SampleSelector);
	InitialValue = AbcImporterUtilities::ConvertAlembicMatrix(MatrixSample.getMatrix());

	return true;
}

void FAbcTransform::SetFrameAndTime(const float InTime, const int32 FrameIndex, const EFrameReadFlags InFlags, const int32 TargetIndex /*= INDEX_NONE*/)
{
	if (TargetIndex != INDEX_NONE)
	{
		InUseSamples[TargetIndex] = true;
		ResidentSampleIndices[TargetIndex] = FrameIndex;
		FrameTimes[TargetIndex] = InTime;

		if (!bConstantIdentity && !bConstant)
		{
			Alembic::Abc::ISampleSelector SampleSelector = AbcImporterUtilities::GenerateAlembicSampleSelector<double>(InTime);
			Alembic::AbcGeom::XformSample MatrixSample;
			Schema.get(MatrixSample, SampleSelector);
			ResidentMatrices[TargetIndex] = AbcImporterUtilities::ConvertAlembicMatrix(MatrixSample.getMatrix());
		}
	}
}

FMatrix FAbcTransform::GetMatrix(const int32 FrameIndex) const
{
	if (bConstantIdentity || bConstant)
	{
		return Parent ? InitialValue * Parent->GetMatrix(FrameIndex) : InitialValue;
	}

	// Find matrix within resident samples
	for (int32 Index = 0; Index < MaxNumberOfResidentSamples; ++Index)
	{
		if (ResidentSampleIndices[Index] == FrameIndex)
		{
			return Parent ? ResidentMatrices[Index] * Parent->GetMatrix(FrameIndex) : ResidentMatrices[Index];
		}
	}

	return Parent ? InitialValue * Parent->GetMatrix(FrameIndex) : InitialValue;
}

bool FAbcTransform::HasConstantTransform() const
{
	return bConstant && (Parent ? Parent->HasConstantTransform() : true);
}

void FAbcTransform::PurgeFrameData(const int32 FrameIndex)
{
	checkf(InUseSamples[FrameIndex], TEXT("Trying to purge a sample which isn't in use"));
	InUseSamples[FrameIndex] = false;
	ResidentSampleIndices[FrameIndex] = INDEX_NONE;
}
