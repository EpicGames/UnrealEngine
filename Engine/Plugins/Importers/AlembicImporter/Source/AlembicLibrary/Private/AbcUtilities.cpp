// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbcUtilities.h"

#include "AbcFile.h"
#include "AbcImportSettings.h"
#include "AbcImportUtilities.h"
#include "GeometryCache.h"
#include "GeometryCacheMeshData.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"

void FAbcUtilities::GetFrameMeshData(FAbcFile& AbcFile, int32 FrameIndex, FGeometryCacheMeshData& OutMeshData, int32 ConcurrencyIndex)
{
	AbcFile.ReadFrame(FrameIndex, EFrameReadFlags::ApplyMatrix, ConcurrencyIndex);


	const UAbcImportSettings* ImportSettings = AbcFile.GetImportSettings();
	checkSlow(ImportSettings);

	float FrameTime = 0.0f;
	FGeometryCacheMeshData MeshData;
	int32 PreviousNumVertices = 0;
	bool bConstantTopology = false;
	const bool bUseVelocitiesAsMotionVectors = ImportSettings->GeometryCacheSettings.MotionVectors == EAbcGeometryCacheMotionVectorsImport::ImportAbcVelocitiesAsMotionVectors;
	const bool bStoreImportedVertexNumbers = ImportSettings->GeometryCacheSettings.bStoreImportedVertexNumbers;

	AbcImporterUtilities::MergePolyMeshesToMeshData(FrameIndex, 0, AbcFile.GetSecondsPerFrame(), bUseVelocitiesAsMotionVectors,
		AbcFile.GetPolyMeshes(), AbcFile.GetLookupMaterialSlot(), FrameTime, MeshData, PreviousNumVertices, bConstantTopology, bStoreImportedVertexNumbers);

	OutMeshData = MoveTemp(MeshData);

	AbcFile.CleanupFrameData(ConcurrencyIndex);
}

void FAbcUtilities::SetupGeometryCacheMaterials(FAbcFile& AbcFile, UGeometryCache* GeometryCache, UObject* Package)
{
	GeometryCache->Materials.Reset();

	const EObjectFlags Flags = RF_Public | RF_Standalone | RF_Transactional;
	UMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);

	for (const FString& FaceSetName : AbcFile.GetUniqueFaceSetNames())
	{
		UMaterialInterface* Material = AbcImporterUtilities::RetrieveMaterial(AbcFile, FaceSetName, Package, Flags);
		GeometryCache->Materials.Add((Material != nullptr) ? Material : DefaultMaterial);		

		if (Material != UMaterial::GetDefaultMaterial(MD_Surface))
		{
			Material->PostEditChange();
		}
	}
}
