// Copyright Epic Games, Inc. All Rights Reserved.

#include "AlembicLibraryModule.h"
#include "AbcImportSettingsCustomization.h"
#include "PropertyEditorModule.h"

LLM_DEFINE_TAG(Alembic);

class FAlembicLibraryModule : public IAlembicLibraryModule
{
	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(Alembic);

		FModuleManager::LoadModuleChecked<IModuleInterface>("GeometryCache");

		// Register class/struct customizations
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RegisterCustomClassLayout("AbcImportSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FAbcImportSettingsCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout("AbcCompressionSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAbcCompressionSettingsCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout("AbcSamplingSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAbcSamplingSettingsCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout("AbcConversionSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAbcConversionSettingsCustomization::MakeInstance));
	}

	virtual void ShutdownModule() override
	{
		// Unregister class/struct customizations
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.UnregisterCustomClassLayout("AbcImportSettings");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("AbcCompressionSettings");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("AbcSamplingSettings");
	}
};

IMPLEMENT_MODULE(FAlembicLibraryModule, AlembicLibrary);
