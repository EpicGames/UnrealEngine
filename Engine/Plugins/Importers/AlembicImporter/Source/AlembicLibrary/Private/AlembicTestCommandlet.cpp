// Copyright Epic Games, Inc. All Rights Reserved.

#include "AlembicTestCommandlet.h"
#include "AbcImportSettings.h"
#include "AbcFile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AlembicTestCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogAlembicCommandlet, Log, All);

/**
 * UAlembicTestCommandlet
 *
 * Commandlet used for testing the alembic importer
 */

UAlembicTestCommandlet::UAlembicTestCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UAlembicTestCommandlet::Main(const FString& Params)
{
	bool bSuccess = false;

 	const TCHAR* ParamStr = *Params;
	ParseCommandLine(ParamStr, CmdLineTokens, CmdLineSwitches);

	if (CmdLineTokens.Num())
	{
		UAbcImportSettings* Settings = GetMutableDefault<UAbcImportSettings>();
		FAbcFile AbcFile(CmdLineTokens[0]);
		AbcFile.Open();

		AbcFile.Import(Settings);
		
		AbcFile.ProcessFrames([&bSuccess](int32 FrameIndex, FAbcFile* File)
		{
			bSuccess = true;
		}, EFrameReadFlags::None);
	}

	FPlatformProcess::Sleep(0.005f);

	return bSuccess ? 0 : 1;
}

