// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "AlembicTestCommandlet.generated.h"

UCLASS()
class UAlembicTestCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	/** Parsed commandline tokens */
	TArray<FString> CmdLineTokens;

	/** Parsed commandline switches */
	TArray<FString> CmdLineSwitches;

	virtual int32 Main(const FString& Params) override;
};
