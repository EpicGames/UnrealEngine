// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

// Just to be sure, also added this in Eigen.Build.cs
#ifndef EIGEN_MPL2_ONLY
#define EIGEN_MPL2_ONLY
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
	#pragma warning(push)
	#pragma warning(disable:6294) // Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed.
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
	#include <Eigen/Dense>
	#include <Eigen/SVD>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
	#pragma warning(pop)
#endif

#define USE_MEMCPY_FOR_EIGEN_CONVERSION 1

namespace EigenHelpers
{
	/** Sets the number of thread used by Eigen */
	static void SetNumEigenThreads(const int32 InNumThreads)
	{
		const int32 CurrentNumThreads = Eigen::nbThreads();
		if (CurrentNumThreads != InNumThreads)
		{
			Eigen::setNbThreads(InNumThreads);
		}
	}

	/** Converts a float array into an Eigen Matrix */
	static void ConvertArrayToEigenMatrix(const TArray64<float>& InArray, const int32 InRows, const int32 InColumns, Eigen::MatrixXf& OutMatrix)
	{
		OutMatrix.resize(InRows, InColumns);

#if USE_MEMCPY_FOR_EIGEN_CONVERSION
		FMemory::Memcpy(OutMatrix.data(), InArray.GetData(), InRows * InColumns * sizeof(float));
#else
		// Copy matrix data
		for (int32 ColumnIndex = 0; ColumnIndex < InColumns; ++ColumnIndex)
		{
			const int64 ColumnOffset = int64(ColumnIndex) * InRows;
			for (int32 RowIndex = 0; RowIndex < InRows; ++RowIndex)
			{
				OutMatrix(RowIndex, ColumnIndex) = InArray[ColumnOffset + RowIndex];
			}
		}
#endif // USE_MEMCPY_FOR_EIGEN_CONVERSION
	}

	/** Converts an Eigen Matrix into a float array */
	static void ConvertEigenMatrixToArray(const Eigen::MatrixXf& InMatrix, TArray64<float>& OutArray, uint32& OutColumns, uint32& OutRows )
	{
		OutColumns = (int32)InMatrix.cols();
		OutRows = (int32)InMatrix.rows();
		const int64 TotalSize = InMatrix.rows() * InMatrix.cols();
		OutArray.Empty(TotalSize);
		OutArray.AddZeroed(TotalSize);

#if USE_MEMCPY_FOR_EIGEN_CONVERSION
		FMemory::Memcpy(OutArray.GetData(), InMatrix.data(), TotalSize * sizeof(float));
#else
		for (int32 ColumnIndex = 0; ColumnIndex < OutColumns; ++ColumnIndex)
		{
			const int64 ColumnOffset = int64(ColumnIndex) * OutRows;
			for (int32 RowIndex = 0; RowIndex < OutRows; ++RowIndex)
			{
				OutArray[ColumnOffset + RowIndex] = InMatrix(RowIndex, ColumnIndex);
			}
		}
#endif // USE_MEMCPY_FOR_EIGEN_CONVERSION
	}
	
	/** Performs Singular value decomposition on the given matrix and returns respective the calculated U, V and S Matrix */
	static void PerformSVD(const Eigen::MatrixXf& InMatrix, Eigen::MatrixXf& OutU, Eigen::MatrixXf& OutV, Eigen::MatrixXf& OutS)
	{
		Eigen::JacobiSVD<Eigen::MatrixXf> SVD(InMatrix, Eigen::ComputeThinU | Eigen::ComputeThinV);
		OutU = SVD.matrixU();
		OutV = SVD.matrixV().transpose();
		OutS = SVD.singularValues();
	}

	/** Performs Singular value decomposition on the given matrix and returns respective the calculated U, V and S Matrix */
	static void PerformSVD(const TArray64<float>& InMatrix, const int32 InRows, const int32 InColumns, TArray64<float>& OutU, TArray64<float>& OutV, TArray64<float>& OutS)
	{
		Eigen::MatrixXf EigenMatrix;
		ConvertArrayToEigenMatrix(InMatrix, InRows, InColumns, EigenMatrix);
		
		Eigen::MatrixXf EigenOutU;
		Eigen::MatrixXf EigenOutV;
		Eigen::MatrixXf EigenOutS;
		PerformSVD(EigenMatrix, EigenOutU, EigenOutV, EigenOutS);

		uint32 NumColumns, NumRows;
		ConvertEigenMatrixToArray(EigenOutU, OutU, NumColumns, NumRows);
		ConvertEigenMatrixToArray(EigenOutV, OutV, NumColumns, NumRows);
		ConvertEigenMatrixToArray(EigenOutS, OutS, NumColumns, NumRows);
	}
};
