// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "EditorFramework/AssetImportData.h"
#include "AbcImportSettings.h"
#include "AbcAssetImportData.generated.h"

/**
* Base class for import data and options used when importing any asset from Alembic
*/
UCLASS(MinimalAPI)
class UAbcAssetImportData : public UAssetImportData
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY()
	TArray<FString> TrackNames;

	UPROPERTY()
	FAbcSamplingSettings SamplingSettings;

	UPROPERTY()
	FAbcNormalGenerationSettings NormalGenerationSettings;

	UPROPERTY()
	FAbcMaterialSettings MaterialSettings;

	UPROPERTY()
	FAbcCompressionSettings CompressionSettings;

	UPROPERTY()
	FAbcStaticMeshSettings StaticMeshSettings;

	UPROPERTY()
	FAbcGeometryCacheSettings GeometryCacheSettings;

	UPROPERTY()
	FAbcConversionSettings ConversionSettings;
};
