// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/BoxSphereBounds.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <Alembic/AbcCoreFactory/IFactory.h>
#include <Alembic/Abc/IArchive.h>
#include <Alembic/Abc/IObject.h>
#include <Alembic/AbcGeom/IPolyMesh.h>
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

class IAbcObject;
class FAbcPolyMesh;
class FAbcTransform;
class UAbcImportSettings;
class UMaterialInterface;
class IMeshUtilities;
struct FSlowTask;

enum EAbcImportError : uint32;

/** Read flags for the abc file */
enum class EFrameReadFlags : uint8
{
	None = 0,
	/** Will only read position and normal data for the objects */
	PositionAndNormalOnly = 1 << 1,
	/** Will pre-multiply the world matrix with the read sample positions */
	ApplyMatrix = 1 << 2,
	/** Will force single thread processing */
	ForceSingleThreaded = 1 << 4,
};

ENUM_CLASS_FLAGS(EFrameReadFlags);

class FAbcFile
{
public:
	ALEMBICLIBRARY_API FAbcFile(const FString& InFilePath);
	ALEMBICLIBRARY_API ~FAbcFile();

	/** Opens the ABC file and reads initial data for import options  */
	ALEMBICLIBRARY_API EAbcImportError Open();
	/** Sets up import data and propagates import settings to read objects */
	ALEMBICLIBRARY_API EAbcImportError Import(UAbcImportSettings* InImportSettings);

	/** Get file path for currently opened ABC file*/
	ALEMBICLIBRARY_API const FString GetFilePath() const;

	/** Process all to-import frames for each poly mesh inside of the ABC file, optional SlowTask to interrupt process and returns 'completed' */
	ALEMBICLIBRARY_API bool ProcessFrames(TFunctionRef<void(int32, FAbcFile*)> InCallback, const EFrameReadFlags InFlags, const FSlowTask* SlowTask = nullptr);
	
	/** Returns ABC specific frame and timing information */
	ALEMBICLIBRARY_API const int32 GetMinFrameIndex() const; // index of the first frame with data regardless of the imported range
	ALEMBICLIBRARY_API const int32 GetMaxFrameIndex() const;
	ALEMBICLIBRARY_API const float GetImportTimeOffset() const;
	ALEMBICLIBRARY_API const float GetImportLength() const;
	ALEMBICLIBRARY_API const int32 GetImportNumFrames() const;
	ALEMBICLIBRARY_API const float GetFramerate() const;
	ALEMBICLIBRARY_API const float GetSecondsPerFrame() const;
	ALEMBICLIBRARY_API const FBoxSphereBounds& GetArchiveBounds() const;
	ALEMBICLIBRARY_API const bool ContainsHeterogeneousMeshes() const;

	/** Returns start/end frame index of imported playback range */
	ALEMBICLIBRARY_API const int32 GetStartFrameIndex() const;
	ALEMBICLIBRARY_API const int32 GetEndFrameIndex() const;

	/** Returns user specified import settings */
	ALEMBICLIBRARY_API const UAbcImportSettings* GetImportSettings() const;	

	/** Return array of PolyMeshes inside of the ABC file */
	ALEMBICLIBRARY_API const TArray<FAbcPolyMesh*>& GetPolyMeshes() const;
	/** Return number of PolyMeshes inside of the ABC file */
	ALEMBICLIBRARY_API const int32 GetNumPolyMeshes() const;
	/** Return array of Transform Objects inside of the ABC file */
	ALEMBICLIBRARY_API const TArray<FAbcTransform*>& GetTransforms() const;

	/** Return mesh utilities module ptr (used in non-gamethread jobs)*/
	ALEMBICLIBRARY_API IMeshUtilities* GetMeshUtilities() const;
	/** Returns a material for the specified material(faceset) name or null if it wasn't created or found */
	ALEMBICLIBRARY_API UMaterialInterface** GetMaterialByName(const FString& InMaterialName);		
	/** Returns frame index equivalent to a given time, based on the frame rate of the file */
	ALEMBICLIBRARY_API int32 GetFrameIndex(float Time);
	/** Reads the frame data for the given FrameIndex, with the ReadIndex used to specify the current read when doing concurrent reads (up to 8 at the same time) */
	ALEMBICLIBRARY_API void ReadFrame(int32 FrameIndex, const EFrameReadFlags InFlags, const int32 ReadIndex = INDEX_NONE);
	/** Cleans up frame data. Must be called after ReadFrame when the frame data (with matching ReadIndex) is not needed anymore */
	ALEMBICLIBRARY_API void CleanupFrameData(const int32 ReadIndex);
	/** Returns the list of unique face set names from the meshes to be imported */
	const TArray<FString>& GetUniqueFaceSetNames() const { return UniqueFaceSetNames; }
	/** Returns the mapping from material index (stored in per triangle material indices of FAbcMeshSample) to material slot (one for each unique faceset name) */
	const TArray<int32>& GetLookupMaterialSlot() const { return LookupMaterialSlot; }
	/** Returns the number of material slots ( == number of unique facesets ) */
	int32 GetNumMaterialSlots() const { return GetUniqueFaceSetNames().Num(); }

	typedef TPair<FString, FString> FMetaData;
	/** Returns the metadata of the Alembic archive */
	ALEMBICLIBRARY_API TArray<FMetaData> GetArchiveMetaData() const;

	static ALEMBICLIBRARY_API const FString NoFaceSetNameStr;

protected:
	ALEMBICLIBRARY_API void TraverseAbcHierarchy(const Alembic::Abc::IObject& InObject, IAbcObject* InParent);
	ALEMBICLIBRARY_API void ExtractCustomAttributes(const Alembic::AbcGeom::IPolyMesh& InMesh);
protected:
	/** File path for the ABC file */
	const FString FilePath;
	/** Cached user set import settings */
	UAbcImportSettings* ImportSettings;

	/** Factory used to generate objects*/
	Alembic::AbcCoreFactory::IFactory Factory;
	Alembic::AbcCoreFactory::IFactory::CoreType CompressionType;
	/** Archive-typed ABC file */
	Alembic::Abc::IArchive Archive;
	/** Alembic typed root (top) object*/
	Alembic::Abc::IObject TopObject;

	/** Abstract and typed objects part of this ABC file*/
	TArray<IAbcObject*> Objects;
	TArray<FAbcPolyMesh*> PolyMeshes;
	TArray<FAbcTransform*> Transforms;	
	/** Root (top) object in the ABC file */
	IAbcObject* RootObject;

	/** Min and maximum frame index which contain actual data in the Alembic file*/
	int32 MinFrameIndex;
	int32 MaxFrameIndex;
	
	/** FPS stored inside of the ABC file */
	float ArchiveSecondsPerFrame;
	
	/** Map of material created for the imported alembic file identified by material names */
	TMap<FString, UMaterialInterface*> MaterialMap;
	TArray<FString> UniqueFaceSetNames;
	TArray<int32> LookupMaterialSlot;

	/** Total (max) number of frames in the Alembic file */
	int32 NumFrames;
	/** Frames per second (retrieved and specified in top Alembic object) */
	float FramesPerSecond;
	/** Seconds per frame (calculated according to FPS) */
	float SecondsPerFrame;

	/** Frame indices at which to start and stop importing */
	int32 StartFrameIndex;
	int32 EndFrameIndex;
	
	/** Entire bounds of the archive over time */
	FBoxSphereBounds ArchiveBounds;

	/** Min and maximum time found in the Alembic file*/
	float MinTime;
	float MaxTime;
	float ImportTimeOffset;
	/** Final length (in seconds)_of sequence we are importing */
	float ImportLength;

	/** Cached Mesh utilities ptr for normal calculations */
	IMeshUtilities* MeshUtilities;

	FString AppName;
	FString LibVersionString;
	uint32 LibVersion;
	FString DateWritten;
	FString UserDescription;
	FString Hash;

	TMap<FString, FString> CustomAttributes;
};
