// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FTokenizedMessage;

class FAbcImportLogger
{
protected:
	FAbcImportLogger();
public:
	/** Adds an import message to the stored array for later output*/
	static void AddImportMessage(const TSharedRef<FTokenizedMessage> Message);
	/** Outputs the messages to a new named page in the message log */
	ALEMBICLIBRARY_API static void OutputMessages(const FString& PageName);
	/** Returns the messages and flushes them from the logger */
	ALEMBICLIBRARY_API static TArray<TSharedRef<FTokenizedMessage>> RetrieveMessages();
private:
	/** Error messages **/
	static TArray<TSharedRef<FTokenizedMessage>> TokenizedErrorMessages;
	static FCriticalSection MessageLock;
};
