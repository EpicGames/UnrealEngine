// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AbcImportSettings.generated.h"

#define UE_API ALEMBICLIBRARY_API

/** Enum that describes type of asset to import */
UENUM(BlueprintType)
enum class EAlembicImportType : uint8
{
	/** Imports only the first frame as one or multiple static meshes */
	StaticMesh,
	/** Imports the Alembic file as flipbook and matrix animated objects */
	GeometryCache UMETA(DisplayName = "Geometry Cache"),
	/** Imports the Alembic file as a skeletal mesh containing base poses as morph targets and blending between them to achieve the correct animation frame */
	Skeletal
};

UENUM(BlueprintType)
enum class EBaseCalculationType : uint8
{
	None = 0 UMETA(Hidden),
	/** Determines the number of bases that should be used with the given percentage*/
	PercentageBased = 1,
	/** Set a fixed number of bases to import*/
	FixedNumber,
	/** One base per frame, uncompressed*/
	NoCompression
};

USTRUCT(Blueprintable)
struct FAbcCompressionSettings
{
	GENERATED_USTRUCT_BODY()

	FAbcCompressionSettings()
	{		
		bMergeMeshes = false;
		bBakeMatrixAnimation = true;

		BaseCalculationType = EBaseCalculationType::PercentageBased;
		PercentageOfTotalBases = 100.0f;
		MaxNumberOfBases = 0;
		MinimumNumberOfVertexInfluencePercentage = 0.0f;
	}

	/** Whether or not the individual meshes should be merged for compression purposes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Compression)
	bool bMergeMeshes;

	/** Whether or not Matrix-only animation should be baked out as vertex animation (or skipped?)*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Compression)
	bool bBakeMatrixAnimation;
	
	/** Determines how the final number of bases that are stored as morph targets are calculated*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Compression)
	EBaseCalculationType BaseCalculationType;

	/** Will generate given percentage of the given bases as morph targets*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Compression, meta = (EnumCondition = 1, ClampMin = "1.0", ClampMax="100.0"))
	float PercentageOfTotalBases;

	/** Will generate given fixed number of bases as morph targets*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Compression, meta = (EnumCondition = 2, ClampMin = "1"))
	int32 MaxNumberOfBases;

	/** Minimum percentage of influenced vertices required for a morph target to be valid */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Compression, meta = (ClampMin = "0.0", ClampMax = "100.0"))
	float MinimumNumberOfVertexInfluencePercentage;
};

UENUM(BlueprintType)
enum class EAlembicSamplingType : uint8
{
	/** Samples the animation according to the imported data (default)*/
	PerFrame,	
	/** Samples the animation at given intervals determined by Frame Steps*/
	PerXFrames UMETA(DisplayName = "Per X Frames"),
	/** Samples the animation at given intervals determined by Time Steps*/
	PerTimeStep
};

USTRUCT(Blueprintable)
struct FAbcSamplingSettings
{
	GENERATED_USTRUCT_BODY()

	FAbcSamplingSettings()
	{
		SamplingType = EAlembicSamplingType::PerFrame;
		FrameSteps = 1;
		TimeSteps = 0.0f;
		FrameStart = FrameEnd = 0;
		bSkipEmpty = false;
	}

	/** Type of sampling performed while importing the animation*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sampling)
	EAlembicSamplingType SamplingType;

	/** Steps to take when sampling the animation*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sampling, meta = (EnumCondition = 1,ClampMin = "1"))
	int32 FrameSteps;

	/** Time steps to take when sampling the animation*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sampling, meta = (EnumCondition = 2, ClampMin = "0.0001"))
	float TimeSteps;
	
	/** Starting index to start sampling the animation from*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sampling)
	int32 FrameStart;

	/** Ending index to stop sampling the animation at*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sampling)
	int32 FrameEnd;

	/** Skip empty (pre-roll) frames and start importing at the frame which actually contains data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sampling, meta=(DisplayName = "Skip Empty Frames at Start of Alembic Sequence"))
	bool bSkipEmpty;
};

USTRUCT(Blueprintable)
struct FAbcNormalGenerationSettings
{
	GENERATED_USTRUCT_BODY()

	FAbcNormalGenerationSettings()
	{
		bRecomputeNormals = false;
		HardEdgeAngleThreshold = 0.9f;
		bForceOneSmoothingGroupPerObject = false;
		bIgnoreDegenerateTriangles = true;
		bSkipComputingTangents = false;
	}

	/** Whether or not to force smooth normals for each individual object rather than calculating smoothing groups */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NormalCalculation)
	bool bForceOneSmoothingGroupPerObject;

	/** Threshold used to determine whether an angle between two normals should be considered hard, closer to 0 means more smooth vs 1 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NormalCalculation, meta = (EditCondition = "!bForceOneSmoothingGroupPerObject", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float HardEdgeAngleThreshold;

	/** Determines whether or not the normals should be forced to be recomputed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NormalCalculation)
	bool bRecomputeNormals;

	/** Determines whether or not the degenerate triangles should be ignored when calculating tangents/normals */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NormalCalculation, meta = (EditCondition = "bRecomputeNormals"))
	bool bIgnoreDegenerateTriangles;

	/** Determines whether tangents are computed for GeometryCache. Skipping them can improve streaming performance but may cause visual artifacts where they are required */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Hidden, AdvancedDisplay)
	bool bSkipComputingTangents;
};

USTRUCT(Blueprintable)
struct FAbcMaterialSettings
{
	GENERATED_USTRUCT_BODY()

	FAbcMaterialSettings()
		: bCreateMaterials(false)
		, bFindMaterials(false)
	{}

	/** Whether or not to create materials according to found Face Set names (will not work without face sets) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Materials)
	bool bCreateMaterials;

	/** Whether or not to try and find materials according to found Face Set names (will not work without face sets) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Materials)
	bool bFindMaterials;
};

USTRUCT(Blueprintable)
struct FAbcStaticMeshSettings
{
	GENERATED_USTRUCT_BODY()

	FAbcStaticMeshSettings()
		: bMergeMeshes(true),
		bPropagateMatrixTransformations(true),
		bGenerateLightmapUVs(true)
	{}
	
	// Whether or not to merge the static meshes on import (remember this can cause problems with overlapping UV-sets)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StaticMesh)
	bool bMergeMeshes;

	// This will, if applicable, apply matrix transformations to the meshes
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StaticMesh)
	bool bPropagateMatrixTransformations;

	// Flag for whether or not lightmap UVs should be generated
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StaticMesh)
	bool bGenerateLightmapUVs;
};

/** Enum that describes type of asset to import */
UENUM(Blueprintable)
enum class EAbcConversionPreset : uint8
{
	/** Imports only the first frame as one or multiple static meshes */
	Maya UMETA(DisplayName = "Autodesk Maya"),
	/** Imports the Alembic file as flipbook and matrix animated objects */
	Max UMETA(DisplayName = "Autodesk 3ds Max"),
	Custom UMETA(DisplayName = "Custom Settings")
};

USTRUCT(Blueprintable)
struct FAbcConversionSettings
{
	GENERATED_USTRUCT_BODY()

	FAbcConversionSettings()
		: Preset(EAbcConversionPreset::Maya)
		, bFlipU(false)
		, bFlipV(true)
		, Scale(FVector(1.0f, -1.0f, 1.0f))
		, Rotation(FVector(90.0f, 0.0f, 0.0f))
	{}

	/** Currently preset that should be applied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Conversion)
	EAbcConversionPreset Preset;

	/** Flag whether or not to flip the U channel in the Texture Coordinates */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Conversion)
	bool bFlipU;

	/** Flag whether or not to flip the V channel in the Texture Coordinates */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Conversion)
	bool bFlipV;

	/** Scale value that should be applied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Conversion)
	FVector Scale;

	/** Rotation in Euler angles that should be applied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Conversion)
	FVector Rotation;
};

UENUM(Blueprintable)
enum class EAbcGeometryCacheMotionVectorsImport : uint8
{
	/** No motion vectors will be present in the geometry cache. */
	NoMotionVectors,
	/** Imports the Velocities from the Alembic file and converts them to motion vectors. This will increase file size as the motion vectors will be stored on disc. */
	ImportAbcVelocitiesAsMotionVectors,
	/** Force calculation of motion vectors during import. This will increase file size as the motion vectors will be stored on disc. */
	CalculateMotionVectorsDuringImport
};

USTRUCT(Blueprintable)
struct FAbcGeometryCacheSettings
{
	GENERATED_USTRUCT_BODY()

	FAbcGeometryCacheSettings()
	:	bFlattenTracks(true),
		bStoreImportedVertexNumbers(false),
		bApplyConstantTopologyOptimizations(false),
		bCalculateMotionVectorsDuringImport_DEPRECATED(false),
		MotionVectors(EAbcGeometryCacheMotionVectorsImport::NoMotionVectors),
		bOptimizeIndexBuffers(false),
		CompressedPositionPrecision(0.01f),
		CompressedTextureCoordinatesNumberOfBits(10)
	{}

	// Whether or not to merge all vertex animation into one track
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GeometryCache)
	bool bFlattenTracks;

	/** Store the imported vertex numbers. This lets you know the vertex numbers inside the DCC.
	  * The values of each vertex number will range from 0 to 7 for a cube. Even if the number of positions might be 24. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GeometryCache, AdvancedDisplay)
	bool bStoreImportedVertexNumbers;

	/** Force the preprocessor to only do optimization once instead of when the preprocessor decides. This may lead to some problems with certain meshes but makes sure motion
	    blur always works if the topology is constant. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GeometryCache)
	bool bApplyConstantTopologyOptimizations;

	/** Force calculation of motion vectors during import. This will increase file size as the motion vectors will be stored on disc. Recommended to OFF.*/
	UPROPERTY()
	bool bCalculateMotionVectorsDuringImport_DEPRECATED;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = GeometryCache)
	EAbcGeometryCacheMotionVectorsImport MotionVectors;

	/** Optimizes index buffers for each unique frame, to allow better cache coherency on the GPU. Very costly and time-consuming process, recommended to OFF.*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = GeometryCache)
	bool bOptimizeIndexBuffers;

	/** Precision used for compressing vertex positions (lower = better result but less compression, higher = more lossy compression but smaller size) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GeometryCache, meta = (ClampMin = "0.000001", ClampMax = "1000", UIMin = "0.01", UIMax = "100"))
	float CompressedPositionPrecision;

	/** Bit-precision used for compressing texture coordinates (hight = better result but less compression, lower = more lossy compression but smaller size) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GeometryCache, meta = (ClampMin = "1", ClampMax = "31", UIMin = "4", UIMax = "16"))
	int32 CompressedTextureCoordinatesNumberOfBits;
};

/** Class that contains all options for importing an alembic file */
UCLASS(MinimalAPI, Blueprintable)
class UAbcImportSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Accessor and initializer **/
	static UE_API UAbcImportSettings* Get();

	/** Type of asset to import from Alembic file */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alembic)
	EAlembicImportType ImportType;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Sampling)
	FAbcSamplingSettings SamplingSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = NormalCalculation)
	FAbcNormalGenerationSettings NormalGenerationSettings;	
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Materials)
	FAbcMaterialSettings MaterialSettings;
		
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Compression)
	FAbcCompressionSettings CompressionSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = StaticMesh)
	FAbcStaticMeshSettings StaticMeshSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = GeometryCache)
	FAbcGeometryCacheSettings GeometryCacheSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Conversion)
	FAbcConversionSettings ConversionSettings;

	bool bReimport;
	int32 NumThreads;

public:
	UE_API virtual void Serialize(class FArchive& Archive) override;
};

#undef UE_API
