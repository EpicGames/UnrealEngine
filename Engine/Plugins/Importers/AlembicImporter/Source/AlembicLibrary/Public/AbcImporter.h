// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshUVChannelInfo.h"
#include "UObject/Object.h" // IWYU pragma: keep

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <Alembic/AbcGeom/All.h>
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

class IAnimationDataController;
class UAnimSequence;
class USkeleton;
class FSkeletalMeshLODModel;
class UMaterial;
class UStaticMesh;
class USkeletalMesh;
class UGeometryCache;
class UDEPRECATED_GeometryCacheTrack_FlipbookAnimation;
class UDEPRECATED_GeometryCacheTrack_TransformAnimation;
class UGeometryCacheTrackStreamable;
class UAbcImportSettings;
class FSkeletalMeshImportData;
class UAbcAssetImportData;

struct FMorphTargetDelta;
struct FReferenceSkeleton;
struct FMeshDescription;
struct FAbcImportData;
struct FGeometryCacheMeshData;
struct FAbcMeshSample;
struct FAbcCompressionSettings;

enum EAbcImportError : uint32
{
	AbcImportError_NoError,
	AbcImportError_InvalidArchive,
	AbcImportError_NoValidTopObject,	
	AbcImportError_NoMeshes,
	AbcImportError_FailedToImportData
};

struct FCompressedAbcData
{
	~FCompressedAbcData();
	/** GUID identifying the poly mesh object this compressed data corresponds to */
	FGuid Guid;
	/** Average sample to apply the bases to */
	FAbcMeshSample* AverageSample;
	/** List of base samples calculated using PCA compression */
	TArray<FAbcMeshSample*> BaseSamples;
	/** Contains the curve values for each individual base */
	TArray<TArray<float>> CurveValues;
	/** Contains the time key values for each individual base */
	TArray<TArray<float>> TimeValues;
};

/** Mesh section used for chunking the mesh data during Skeletal mesh building */
struct FMeshSection
{
	FMeshSection() : MaterialIndex(0), NumFaces(0) {}
	int32 MaterialIndex;
	TArray<uint32> Indices;
	TArray<uint32> OriginalIndices;
	TArray<FVector> TangentX;
	TArray<FVector> TangentY;
	TArray<FVector> TangentZ;
	TArray<FVector2D> UVs[MAX_TEXCOORDS];
	TArray<FColor> Colors;
	uint32 NumFaces;
	uint32 NumUVSets;
};

class FAbcImporter
{
public:
	ALEMBICLIBRARY_API FAbcImporter();
	ALEMBICLIBRARY_API ~FAbcImporter();
public:
	/**
	* Opens and caches basic data from the Alembic file to be used for populating the importer UI
	* 
	* @param InFilePath - Path to the Alembic file to be opened
	* @return - Possible error code 
	*/
	ALEMBICLIBRARY_API const EAbcImportError OpenAbcFileForImport(const FString InFilePath);

	/**
	* Imports the individual tracks from the Alembic file
	*
	* @param InNumThreads - Number of threads to use for importing
	* @return - Possible error code
	*/
	ALEMBICLIBRARY_API const EAbcImportError ImportTrackData(const int32 InNumThreads, UAbcImportSettings* ImportSettings);
	
	/**
	* Import Alembic meshes as a StaticMeshInstance
	*	
	* @param InParent - ParentObject for the static mesh
	* @param Flags - Object flags
	* @return FStaticMesh*
	*/
	ALEMBICLIBRARY_API const TArray<UStaticMesh*> ImportAsStaticMesh(UObject* InParent, EObjectFlags Flags);

	/**
	* Import an Alembic file as a GeometryCache
	*	
	* @param InParent - ParentObject for the static mesh
	* @param Flags - Object flags
	* @return UGeometryCache*
	*/
	ALEMBICLIBRARY_API UGeometryCache* ImportAsGeometryCache(UObject* InParent, EObjectFlags Flags);

	ALEMBICLIBRARY_API TArray<UObject*> ImportAsSkeletalMesh(UObject* InParent, EObjectFlags Flags);	

	/**
	* Reimport an Alembic mesh
	*
	* @param Mesh - Current StaticMesh instance
	* @return FStaticMesh*
	*/
	ALEMBICLIBRARY_API const TArray<UStaticMesh*> ReimportAsStaticMesh(UStaticMesh* Mesh);

	/**
	* Reimport an Alembic file as a GeometryCache
	*
	* @param GeometryCache - Current GeometryCache instance
	* @return UGeometryCache*
	*/
	ALEMBICLIBRARY_API UGeometryCache* ReimportAsGeometryCache(UGeometryCache* GeometryCache);

	/**
	* Reimport an Alembic file as a SkeletalMesh
	*
	* @param SkeletalMesh - Current SkeletalMesh instance
	* @return USkeletalMesh*
	*/
	ALEMBICLIBRARY_API TArray<UObject*> ReimportAsSkeletalMesh(USkeletalMesh* SkeletalMesh);

	/** Returns the array of imported PolyMesh objects */
	ALEMBICLIBRARY_API const TArray<class FAbcPolyMesh*>& GetPolyMeshes() const;
	
	/** Returns the lowest frame index containing data for the imported Alembic file */
	ALEMBICLIBRARY_API const uint32 GetStartFrameIndex() const;

	/** Returns the highest frame index containing data for the imported Alembic file */
	ALEMBICLIBRARY_API const uint32 GetEndFrameIndex() const;

	/** Returns the number of tracks found in the imported Alembic file */
	ALEMBICLIBRARY_API const uint32 GetNumMeshTracks() const;

	ALEMBICLIBRARY_API void UpdateAssetImportData(UAbcAssetImportData* AssetImportData);
	ALEMBICLIBRARY_API void RetrieveAssetImportData(UAbcAssetImportData* ImportData);
private:
	/**
	* Creates an template object instance taking into account existing Instances and Objects (on reimporting)
	*
	* @param InParent - ParentObject for the geometry cache, this can be changed when parent is deleted/re-created
	* @param ObjectName - Name to be used for the created object
	* @param Flags - Object creation flags
	*/
	template<typename T> T* CreateObjectInstance(UObject*& InParent, const FString& ObjectName, const EObjectFlags Flags, bool& bObjectAlreadyExists);
	
	/**
	* Creates a Static mesh from the given mesh description
	*
	* @param InParent - ParentObject for the static mesh
	* @param Name - Name for the static mesh
	* @param Flags - Object flags
	* @param NumMaterials - Number of materials to add
	* @param UniqueFaceSetNames - The array of unique face set (material slot) names for merged mesh
	* @param LookupMaterialSlot - Mapping from faceset index (in flattened list) to material slot
	* @param Sample - The FAbcMeshSample from which the static mesh should be constructed
	* @return UStaticMesh*
	*/
	ALEMBICLIBRARY_API UStaticMesh* CreateStaticMeshFromSample(UObject* InParent, const FString& Name, EObjectFlags Flags, const TArray<FString>& UniqueFaceSetNames, const TArray<int32>& LookupMaterialSlot, const FAbcMeshSample* Sample);

	/** Generates and populate a FMeshDescription instance from the given sample*/
	ALEMBICLIBRARY_API void GenerateMeshDescriptionFromSample(const TArray<FString>& UniqueFaceSetNames, const TArray<int32>& LookupMaterialSlot, const FAbcMeshSample* Sample, FMeshDescription* MeshDescription);
	
	/** Compresses the imported animation data, returns true if compression was successful and compressed data was populated */
	ALEMBICLIBRARY_API const bool CompressAnimationDataUsingPCA(const FAbcCompressionSettings& InCompressionSettings, const bool bRunComparison = false);	
	/** Performs the actual SVD compression to retrieve the bases and weights used to set up the Skeletal mesh's morph targets */
	ALEMBICLIBRARY_API const int32 PerformSVDCompression(const TArray64<float>& OriginalMatrix, const TArray64<float>& OriginalNormalsMatrix, const uint32 NumSamples, const float InPercentage, const int32 InFixedNumValue,
		TArray64<float>& OutU, TArray64<float>& OutNormalsU, TArray64<float>& OutV);

	/** Functionality for comparing the matrices and calculating the difference from the original animation */
	ALEMBICLIBRARY_API void CompareCompressionResult(const TArray64<float>& OriginalMatrix, const uint32 NumSamples, const uint32 NumUsedSingularValues, const TArrayView64<float>& OutU, const TArray64<float>& OutV, const float Tolerance);
	
	/** Build a skeletal mesh from the PCA compressed data */
	ALEMBICLIBRARY_API bool BuildSkeletalMesh(FSkeletalMeshLODModel& LODModel, const FReferenceSkeleton& RefSkeleton, FAbcMeshSample* Sample, 
		int32 NumMaterialSlots, const TArray<int32> LookupMaterialSlot, 
		TArray<int32>& OutMorphTargetVertexRemapping, TArray<int32>& OutUsedVertexIndicesForMorphs);
	
	/** Generate morph target vertices from the PCA compressed bases */
	ALEMBICLIBRARY_API void GenerateMorphTargetVertices(FAbcMeshSample* BaseSample, TArray<FMorphTargetDelta> &MorphDeltas, FAbcMeshSample* AverageSample, uint32 WedgeOffset, const TArray<int32>& RemapIndices, const TArray<int32>& UsedVertexIndicesForMorphs, const uint32 VertexOffset, const uint32 IndexOffset);
	
	/** Set up correct morph target weights from the PCA compressed data */
	ALEMBICLIBRARY_API void SetupMorphTargetCurves(USkeleton* Skeleton, FName ConstCurveName, UAnimSequence* Sequence, const TArray<float> &CurveValues, const TArray<float>& TimeValues, IAnimationDataController& Controller, bool bShouldTransact);
	
	/** Set the Alembic archive metadata on the given objects */
	ALEMBICLIBRARY_API void SetMetaData(const TArray<UObject*>& Objects);

private:
	/** Cached ptr for the import settings */
	UAbcImportSettings* ImportSettings;

	/** Resulting compressed data from PCA compression */
	TArray<FCompressedAbcData> CompressedMeshData;

	/** Offset for each sample, used as the root bone translation */
	TOptional<TArray<FVector>> SamplesOffsets;

	/** ABC file representation for currently opened filed */
	class FAbcFile* AbcFile;
};
