// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Matrix.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <Alembic/Abc/IObject.h>
#include <Alembic/AbcGeom/All.h>
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

class FAbcFile;
enum class EFrameReadFlags : uint8;

static const int32 MaxNumberOfResidentSamples = 8;

class IAbcObject
{
public:
	ALEMBICLIBRARY_API IAbcObject(const Alembic::Abc::IObject& InObject, const FAbcFile* InFile, IAbcObject* InParent);
	virtual ~IAbcObject() = default;

	const FString& GetName() const { return Name; }
	ALEMBICLIBRARY_API float GetTimeForFrameIndex(const int32 FrameIndex) const;
	float GetTimeForFirstData() const { return MinTime; }
	float GetTimeForLastData() const { return MaxTime; }
	int32 GetFrameIndexForFirstData() const { return StartFrameIndex;  }
	int32 GetNumberOfSamples() const { return NumSamples;  }
	bool IsConstant() const { return bConstant; }

	virtual FMatrix GetMatrix(const int32 FrameIndex) const { return FMatrix::Identity; };
	virtual void SetFrameAndTime(const float InTime, const int32 FrameIndex, const EFrameReadFlags InFlags, const int32 TargetIndex = INDEX_NONE) = 0;
	virtual bool HasConstantTransform() const = 0;
	virtual bool ReadFirstFrame(const float InTime, const int32 FrameIndex) = 0;
	virtual void PurgeFrameData(const int32 FrameIndex) = 0;
protected:
	/** Name of this object */
	FString Name;
	
	/** Parent object */
	IAbcObject* Parent;	

	/** File of which this object is part of */
	const FAbcFile* File;
	/** Abstract Alembic representation of this object */
	const Alembic::Abc::IObject Object;

	/** Flag whether or not this object is constant */
	bool bConstant;

	/** Time of first frame containing data */
	float MinTime;
	/** Time of last frame containing data */
	float MaxTime;
	/** Frame index of first frame containing data */
	int32 StartFrameIndex;

	/** Number of data samples for this object */	
	int32 NumSamples;

	float FrameTimes[MaxNumberOfResidentSamples];
	int32 ResidentSampleIndices[MaxNumberOfResidentSamples];
	bool InUseSamples[MaxNumberOfResidentSamples];
};
