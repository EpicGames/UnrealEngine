// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AbcObject.h"
#include "Math/BoxSphereBounds.h"
#include "MeshUVChannelInfo.h"

#define UE_API ALEMBICLIBRARY_API

enum class ESampleReadFlags : uint8;

/** Structure for storing individual track samples */
struct FAbcMeshSample
{
	FAbcMeshSample() : NumSmoothingGroups(0)
		, NumUVSets(1)
		, NumMaterials(0)
		, SampleTime(0.0f)
	{}

	/** Constructing from other sample*/
	FAbcMeshSample(const FAbcMeshSample& InSample)
	{
		Vertices = InSample.Vertices;
		Velocities = InSample.Velocities;
		Indices = InSample.Indices;
		Normals = InSample.Normals;
		TangentX = InSample.TangentX;
		TangentY = InSample.TangentY;
		for (uint32 UVIndex = 0; UVIndex < InSample.NumUVSets; ++UVIndex)
		{
			UVs[UVIndex] = InSample.UVs[UVIndex];
		}
		Colors = InSample.Colors;
		/*Visibility = InSample.Visibility;
		VisibilityIndices = InSample.VisibilityIndices;*/
		MaterialIndices = InSample.MaterialIndices;
		SmoothingGroupIndices = InSample.SmoothingGroupIndices;
		NumSmoothingGroups = InSample.NumSmoothingGroups;
		NumMaterials = InSample.NumMaterials;
		SampleTime = InSample.SampleTime;
		NumUVSets = InSample.NumUVSets;
	}

	void Reset(const ESampleReadFlags ReadFlags);
	void Copy(const FAbcMeshSample& InSample, const ESampleReadFlags ReadFlags);
	void Copy(const FAbcMeshSample* InSample, const ESampleReadFlags ReadFlags);

	TArray<FVector3f> Vertices;
	TArray<FVector3f> Velocities;
	TArray<uint32> Indices;

	// Vertex attributes (per index based)
	TArray<FVector3f> Normals;
	TArray<FVector3f> TangentX;
	TArray<FVector3f> TangentY;
	TArray<FVector2f> UVs[MAX_TEXCOORDS];

	TArray<FLinearColor> Colors;
	/*TArray<FVector2D> Visibility;
	TArray<uint32> VisibilityIndices;*/

	// Per Face material and smoothing group index
	TArray<int32> MaterialIndices;
	TArray<uint32> SmoothingGroupIndices;

	/** Number of smoothing groups and different materials (will always be at least 1) */
	uint32 NumSmoothingGroups;
	uint32 NumUVSets;
	uint32 NumMaterials;

	// Time in track this sample was taken from
	float SampleTime;
};

class FAbcPolyMesh : public IAbcObject
{
public:
	UE_API FAbcPolyMesh(const Alembic::AbcGeom::IPolyMesh& InPolyMesh, const FAbcFile* InFile, IAbcObject* InParent = nullptr);
	virtual ~FAbcPolyMesh() = default;

	/** Begin IAbcObject overrides */
	UE_API virtual bool ReadFirstFrame(const float InTime, const int32 FrameIndex) final;
	UE_API virtual void SetFrameAndTime(const float InTime, const int32 FrameIndex, const EFrameReadFlags InFlags, const int32 TargetIndex = INDEX_NONE) final;
	UE_API virtual FMatrix GetMatrix(const int32 FrameIndex) const final;
	UE_API virtual bool HasConstantTransform() const final;
	UE_API virtual void PurgeFrameData(const int32 FrameIndex) final;
	/** End IAbcObject overrides */
	
	/** Returns sample for the given frame index (if it is part of the resident samples) */
	UE_API const FAbcMeshSample* GetSample(const int32 FrameIndex) const;
	/** Returns the first sample available for this object */
	UE_API const FAbcMeshSample* GetFirstSample() const;
	/** Returns the first sample available for this object transformed by first available matrix */
	UE_API const FAbcMeshSample* GetTransformedFirstSample() const;
	/** Returns the value of the bitmask used for skipping constant vertex attributes while reading samples*/
	UE_API ESampleReadFlags GetSampleReadFlags() const;
	/** Returns the visibility value (true = visible, false = hidden) for the given frame index  (if it is part of the resident samples)*/
	UE_API const bool GetVisibility(const int32 FrameIndex) const;	
	
	/** Flag whether or not this object has constant topology (used for eligibility for PCA compression) */
	bool bConstantTopology;
	/** Flag whether or not this object has a constant world matrix (used whether to incorporate into PCA compression) */
	bool bConstantTransformation;
	/** Flag whether or not this object has a constant visibility value across the entire animated range */
	bool bConstantVisibility;

	/** Cached self and child bounds for entire duration of the animation */
	FBoxSphereBounds SelfBounds;
	FBoxSphereBounds ChildBounds;

	/** Array of face set names found for this object */
	TArray<FString> FaceSetNames;

	/** Whether or not this Mesh object should be imported */
	bool bShouldImport;
protected:
	/** Calculate normals for sample according to available data and user settings */
	UE_API void CalculateNormalsForSample(FAbcMeshSample* Sample);

	/** Alembic representation of this object */
	const Alembic::AbcGeom::IPolyMesh PolyMesh;
	/** Schema extracted from Poly Mesh object  */
	const Alembic::AbcGeom::IPolyMeshSchema Schema;

	/** Initial mesh sample for this object in first frame with available data */
	FAbcMeshSample* FirstSample;
	/** Initial mesh sample for this object in first frame with available data, transformed by first available matrix */
	FAbcMeshSample* TransformedFirstSample;
	/** Resident set of Mesh Samples for this object, used for parallel reading of samples/frames */
	FAbcMeshSample* ResidentSamples[MaxNumberOfResidentSamples];
	bool ResidentVisibilitySamples[MaxNumberOfResidentSamples];
	/** Bitmask read flag used for skipping constant vertex attributes while reading samples */
	ESampleReadFlags SampleReadFlags;

	/** Whether or not to just return FirstSample */
	bool bReturnFirstSample;
	/** Whether or not to just return the transformed FirstSample */
	bool bReturnTransformedFirstSample;
	/** Whether or not the mesh is visible in the first Frame */
	bool bFirstFrameVisibility;
};

#undef UE_API
