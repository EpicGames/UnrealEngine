// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AbcObject.h"

#define UE_API ALEMBICLIBRARY_API

class FAbcTransform : public IAbcObject
{
public:
	UE_API FAbcTransform(const Alembic::AbcGeom::IXform& InTransform, const FAbcFile* InFile, IAbcObject* InParent = nullptr);
	virtual ~FAbcTransform() = default;

	/** Begin IAbcObject overrides */
	UE_API virtual bool ReadFirstFrame(const float InTime, const int32 FrameIndex) final;
	UE_API virtual void SetFrameAndTime(const float InTime, const int32 FrameIndex, const EFrameReadFlags InFlags, const int32 TargetIndex = INDEX_NONE) final;
	UE_API virtual FMatrix GetMatrix(const int32 FrameIndex) const final;
	UE_API virtual bool HasConstantTransform() const final;
	UE_API virtual void PurgeFrameData(const int32 FrameIndex) final;
	/** End IAbcObject overrides */	
public:
	/** Flag whether or not this transformation object is identity constant */
	bool bConstantIdentity;
protected:
	/** Alembic representation of this object */
	const Alembic::AbcGeom::IXform Transform;
	/** Schema extracted from transform object  */
	const Alembic::AbcGeom::IXformSchema Schema;

	/** Initial value for this object in first frame with available data */
	FMatrix InitialValue;
	/** Resident set of matrix values for this object, used for parallel reading of samples/frames */
	FMatrix ResidentMatrices[MaxNumberOfResidentSamples];
};

#undef UE_API
