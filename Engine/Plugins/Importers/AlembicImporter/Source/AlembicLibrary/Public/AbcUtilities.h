// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API ALEMBICLIBRARY_API

class UObject;

class FAbcFile;
struct FGeometryCacheMeshData;

class FAbcUtilities
{
public:
	/*  Populates a FGeometryCacheMeshData instance (with merged meshes) for the given frame of an Alembic 
	 *  with ConcurrencyIndex used as the concurrent read index
	 */
	static UE_API void GetFrameMeshData(FAbcFile& AbcFile, int32 FrameIndex, FGeometryCacheMeshData& OutMeshData, int32 ConcurrencyIndex = 0);

	/** Sets up materials from an AbcFile to a GeometryCache and moves them into the given package */
	static UE_API void SetupGeometryCacheMaterials(FAbcFile& AbcFile, class UGeometryCache* GeometryCache, UObject* Package);
};

#undef UE_API
